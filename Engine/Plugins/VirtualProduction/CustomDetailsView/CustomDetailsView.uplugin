{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Custom Details View",
	"Description": "",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "CustomDetailsView",
			"Type": "Editor",
			"LoadingPhase": "PostDefault"
		}
	]
}
