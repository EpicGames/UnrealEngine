// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomDetailsViewModule.h"
#include "CustomDetailsViewRowExtensions.h"
#include "SCustomDetailsView.h"

class FCustomDetailsViewModule : public ICustomDetailsViewModule
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin ICustomDetailsViewModule
	virtual TSharedRef<ICustomDetailsView> CreateCustomDetailsView(const FCustomDetailsViewArgs& InArgs) override;
	//~ End ICustomDetailsViewModule
};

IMPLEMENT_MODULE(FCustomDetailsViewModule, CustomDetailsView)

void FCustomDetailsViewModule::StartupModule()
{
	using namespace UE::CustomDetailsView::Private;

	FCustomDetailsViewRowExtensions::Get().RegisterRowExtensions();
}

void FCustomDetailsViewModule::ShutdownModule()
{
	using namespace UE::CustomDetailsView::Private;

	FCustomDetailsViewRowExtensions::Get().UnregisterRowExtensions();
}

TSharedRef<ICustomDetailsView> FCustomDetailsViewModule::CreateCustomDetailsView(const FCustomDetailsViewArgs& InArgs)
{
	// log in case there's a misconfiguration where keyframe handling is set but global extensions is not set to true
	ensureMsgf(InArgs.bAllowGlobalExtensions || !InArgs.KeyframeHandler.IsValid()
		, TEXT("A Keyframe Handler was specified, but bAllowGlobalExtensions is false"));

	return SNew(SCustomDetailsView, InArgs);
}
