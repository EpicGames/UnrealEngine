// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomDetailsViewRowExtensions.h"

#include "CustomDetailsViewMenuContext.h"
#include "DetailRowMenuContext.h"
#include "PropertyEditorClipboard.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Styling/CoreStyle.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "CustomDetailsViewRowExtensions"

namespace UE::CustomDetailsView::Private
{
	const FName RowExtensionName = "CustomDetailsViewRowExtensionContextSection";
	const FName EditMenuName = "Edit";
	const FName MenuEntry_Copy = "Copy";
	const FName MenuEntry_Paste = "Paste";
	const FName PropertyEditorModuleName = "PropertyEditor";

	FText GetPropertyDisplayName(TSharedRef<IPropertyHandle> InPropertyHandle)
	{
		if (InPropertyHandle->AsOptional().IsValid())
		{
			if (TSharedPtr<IPropertyHandle> ParentHandle = InPropertyHandle->GetParentHandle())
			{
				return ParentHandle->GetPropertyDisplayName();
			}
		}

		return InPropertyHandle->GetPropertyDisplayName();
	}
}

FCustomDetailsViewRowExtensions& FCustomDetailsViewRowExtensions::Get()
{
	static FCustomDetailsViewRowExtensions Instance;
	return Instance;
}

FCustomDetailsViewRowExtensions::~FCustomDetailsViewRowExtensions()
{
	UnregisterRowExtensions();
}

void FCustomDetailsViewRowExtensions::RegisterRowExtensions()
{
	using namespace UE::CustomDetailsView::Private;

	FPropertyEditorModule& Module = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
	FOnGenerateGlobalRowExtension& RowExtensionDelegate = Module.GetGlobalRowExtensionDelegate();
	RowExtensionHandle = RowExtensionDelegate.AddStatic(&FCustomDetailsViewRowExtensions::HandleCreatePropertyRowExtension);
}

void FCustomDetailsViewRowExtensions::UnregisterRowExtensions()
{
	using namespace UE::CustomDetailsView::Private;

	if (RowExtensionHandle.IsValid() && FModuleManager::Get().IsModuleLoaded(PropertyEditorModuleName))
	{
		FPropertyEditorModule& Module = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
		Module.GetGlobalRowExtensionDelegate().Remove(RowExtensionHandle);
		RowExtensionHandle.Reset();
	}
}

void FCustomDetailsViewRowExtensions::HandleCreatePropertyRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions)
{
	using namespace UE::CustomDetailsView::Private;

	if (!InArgs.Property && !InArgs.PropertyHandle.IsValid())
	{
		return;
	}

	UToolMenus* Menus = UToolMenus::Get();
	check(Menus);

	UToolMenu* ContextMenu = Menus->FindMenu(UE::PropertyEditor::RowContextMenuName);

	if (ContextMenu->ContainsSection(RowExtensionName))
	{
		return;
	}

	ContextMenu->AddDynamicSection(
		RowExtensionName,
		FNewToolMenuDelegate::CreateStatic(&FCustomDetailsViewRowExtensions::FillPropertyRightClickMenu)
	);
}

void FCustomDetailsViewRowExtensions::FillPropertyRightClickMenu(UToolMenu* InToolMenu)
{
	using namespace UE::CustomDetailsView::Private;

	if (!InToolMenu->FindContext<UCustomDetailsViewMenuContext>())
	{
		return;
	}

	UDetailRowMenuContext* RowMenuContext = InToolMenu->FindContext<UDetailRowMenuContext>();

	if (!RowMenuContext)
	{
		return;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle;

	for (const TSharedPtr<IPropertyHandle>& ContextPropertyHandle : RowMenuContext->PropertyHandles)
	{
		if (ContextPropertyHandle.IsValid())
		{
			PropertyHandle = ContextPropertyHandle;
			break;
		}
	}

	if (!PropertyHandle.IsValid())
	{
		return;
	}

	FUIAction CopyAction;
	FUIAction PasteAction;

	PropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);

	const bool bCanCopy = CopyAction.ExecuteAction.IsBound();
	bool bCanPaste = true;

	if (PropertyHandle->IsEditConst() || !PropertyHandle->IsEditable())
	{
		bCanPaste = false;
	}
	else if (!PasteAction.ExecuteAction.IsBound())
	{
		bCanPaste = false;
	}

	if (!bCanCopy && !bCanPaste)
	{
		return;
	}

	FToolMenuSection& Section = InToolMenu->AddSection(EditMenuName, LOCTEXT("Edit", "Edit"));

	constexpr bool bLongDisplayName = false;

	if (bCanCopy)
	{
		const TAttribute<FText> Label = LOCTEXT("CopyProperty", "Copy");
		const TAttribute<FText> ToolTip = LOCTEXT("CopyProperty_ToolTip", "Copy this property value");

		FToolMenuEntry& CopyMenuEntry = Section.AddMenuEntry(
			TEXT("Copy"),
			Label,
			ToolTip,
			FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
			CopyAction
		);

		CopyMenuEntry.InputBindingLabel = FInputChord(EModifierKey::Shift, EKeys::RightMouseButton).GetInputText(bLongDisplayName);
	}

	if (bCanPaste)
	{
		PasteAction.ExecuteAction.BindLambda(
			[OriginalAction = PasteAction.ExecuteAction, PropertyHandleWeak = PropertyHandle.ToWeakPtr()]
			{
				if (!OriginalAction.IsBound())
				{
					return;
				}

				TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

				if (!PropertyHandle.IsValid())
				{
					return;
				}

				// This is not part of the default paste action.
				FScopedTransaction Transaction(LOCTEXT("PastePropertyTransaction", "Paste Property"));

				TArray<UObject*> Objects;
				PropertyHandle->GetOuterObjects(Objects);

				for (UObject* Object : Objects)
				{
					Object->Modify();
				}

				OriginalAction.Execute();
			}
		);

		const TAttribute<FText> Label = LOCTEXT("PasteProperty", "Paste");
		const TAttribute<FText> ToolTip = LOCTEXT("PasteProperty_ToolTip", "Paste the copied value here");

		FToolMenuEntry& PasteMenuEntry = Section.AddMenuEntry(
			TEXT("Paste"),
			Label,
			ToolTip,
			FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Paste"),
			PasteAction
		);

		PasteMenuEntry.InputBindingLabel = FInputChord(EModifierKey::Shift, EKeys::LeftMouseButton).GetInputText(bLongDisplayName);
	}

	// Copy Display Name
	{
		const FText PropertyDisplayName = GetPropertyDisplayName(PropertyHandle.ToSharedRef());

		FUIAction CopyDisplayNameAction = FExecuteAction::CreateLambda(
			[PropertyDisplayName]()
			{
				FPropertyEditorClipboard::ClipboardCopy(*PropertyDisplayName.ToString());
			});

		static const FTextFormat TooltipFormat = NSLOCTEXT("PropertyView_Single", "CopyPropertyDisplayName_ToolTip", "Copy the display name of this property to the system clipboard:\n{0}");

		Section.AddMenuEntry(
			TEXT("CopyDisplayName"),
			NSLOCTEXT("PropertyView", "CopyPropertyDisplayName", "Copy Display Name"),
			FText::Format(TooltipFormat, PropertyDisplayName),
			FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
			CopyDisplayNameAction
		);
	}
}

#undef LOCTEXT_NAMESPACE
