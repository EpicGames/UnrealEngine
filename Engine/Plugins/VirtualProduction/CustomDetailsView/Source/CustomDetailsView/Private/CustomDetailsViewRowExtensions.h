// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/IDelegateInstance.h"

class UToolMenu;
struct FOnGenerateGlobalRowExtensionArgs;
struct FPropertyRowExtensionButton;

class FCustomDetailsViewRowExtensions
{
public:
	static FCustomDetailsViewRowExtensions& Get();

	~FCustomDetailsViewRowExtensions();

	void RegisterRowExtensions();

	void UnregisterRowExtensions();

private:
	FDelegateHandle RowExtensionHandle;

	static void HandleCreatePropertyRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions);

	static void FillPropertyRightClickMenu(UToolMenu* InToolMenu);
};
