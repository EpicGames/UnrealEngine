// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomDetailsViewSequencer.h"
#include "Containers/Array.h"
#include "Framework/Commands/UIAction.h"
#include "IDetailKeyframeHandler.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

namespace UE::CustomDetailsView::Private
{

static const TMap<EPropertyKeyedStatus, FName> KeyedStatusStyleNames =
{
	{ EPropertyKeyedStatus::NotKeyed, "Sequencer.KeyedStatus.NotKeyed" },
	{ EPropertyKeyedStatus::KeyedInOtherFrame, "Sequencer.KeyedStatus.Animated" },
	{ EPropertyKeyedStatus::KeyedInFrame, "Sequencer.KeyedStatus.Keyed" },
	{ EPropertyKeyedStatus::PartiallyKeyed, "Sequencer.KeyedStatus.PartialKey" },
};

FSlateIcon GetKeyframeIcon(FCustomDetailsViewSequencerUtils::FGetKeyframeHandlerDelegate InKeyframeHandlerDelegate, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	if (!InPropertyHandle.IsValid())
	{
		return FSlateIcon();
	}

	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;
	if (TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = InKeyframeHandlerDelegate.Execute())
	{
		KeyedStatus = KeyframeHandler->GetPropertyKeyedStatus(*InPropertyHandle);
	}

	const FName* FoundIcon = KeyedStatusStyleNames.Find(KeyedStatus);
	if (!ensure(FoundIcon))
	{
		FoundIcon = &KeyedStatusStyleNames[EPropertyKeyedStatus::NotKeyed];
	}
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), *FoundIcon);
}

void OnAddKeyframeClicked(FCustomDetailsViewSequencerUtils::FGetKeyframeHandlerDelegate InKeyframeHandlerDelegate, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = InKeyframeHandlerDelegate.Execute();
	if (KeyframeHandler.IsValid() && PropertyHandle.IsValid())
	{
		KeyframeHandler->OnKeyPropertyClicked(*PropertyHandle);
	}
}

bool IsKeyframeButtonEnabled(FCustomDetailsViewSequencerUtils::FGetKeyframeHandlerDelegate InKeyframeHandlerDelegate)
{
	if (TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = InKeyframeHandlerDelegate.Execute())
	{
		return KeyframeHandler->IsPropertyKeyingEnabled();
	}
	return false;
}

bool IsKeyframeButtonVisible(FCustomDetailsViewSequencerUtils::FGetKeyframeHandlerDelegate InKeyframeHandlerDelegate, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = InKeyframeHandlerDelegate.Execute();
	if (!KeyframeHandler.IsValid() || !PropertyHandle.IsValid())
	{
		return false;
	}

	const UClass* ObjectClass = PropertyHandle->GetOuterBaseClass();
	if (!ObjectClass)
	{
		return false;
	}

	return KeyframeHandler->IsPropertyKeyable(ObjectClass, *PropertyHandle);
}

}

void FCustomDetailsViewSequencerUtils::CreateSequencerExtensionButton(const FGetKeyframeHandlerDelegate& InKeyframeHandlerDelegate,
	TSharedPtr<IPropertyHandle> InPropertyHandle, TArray<FPropertyRowExtensionButton>& OutExtensionButtons)
{
	if (!InKeyframeHandlerDelegate.IsBound() || !InPropertyHandle.IsValid())
	{
		return;
	}

	using namespace UE::CustomDetailsView::Private;

	FPropertyRowExtensionButton& CreateKey = OutExtensionButtons.AddDefaulted_GetRef();
	CreateKey.Icon = TAttribute<FSlateIcon>::Create(TAttribute<FSlateIcon>::FGetter::CreateStatic(&GetKeyframeIcon, InKeyframeHandlerDelegate, InPropertyHandle));
	CreateKey.Label = NSLOCTEXT("PropertyEditor", "CreateKey", "Create Key");
	CreateKey.ToolTip = NSLOCTEXT("PropertyEditor", "CreateKeyToolTip", "Add a keyframe for this property.");
	CreateKey.UIAction = FUIAction(FExecuteAction::CreateStatic(&OnAddKeyframeClicked, InKeyframeHandlerDelegate, InPropertyHandle)
		, FCanExecuteAction::CreateStatic(&IsKeyframeButtonEnabled, InKeyframeHandlerDelegate)
		, FGetActionCheckState()
		, FIsActionButtonVisible::CreateStatic(&IsKeyframeButtonVisible, InKeyframeHandlerDelegate, InPropertyHandle));
}

void FCustomDetailsViewSequencerUtils::CreateSequencerExtensionButton(TWeakPtr<IDetailKeyframeHandler> InKeyframeHandlerWeak, TSharedPtr<IPropertyHandle> InPropertyHandle, TArray<FPropertyRowExtensionButton>& OutExtensionButtons)
{
	const FCustomDetailsViewSequencerUtils::FGetKeyframeHandlerDelegate KeyframeHandlerDelegate =
		FCustomDetailsViewSequencerUtils::FGetKeyframeHandlerDelegate::CreateLambda(
			[InKeyframeHandlerWeak]() { return InKeyframeHandlerWeak.Pin(); }
	);

	CreateSequencerExtensionButton(KeyframeHandlerDelegate, InPropertyHandle, OutExtensionButtons);
}
