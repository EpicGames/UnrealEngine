// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomDetailsViewCustomCategoryItem.h"
#include "Internationalization/Text.h"
#include "CustomDetailsViewItemBase.h"
#include "SCustomDetailsView.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CustomDetailsViewCustomCategoryItem"

FCustomDetailsViewCustomCategoryItem::FCustomDetailsViewCustomCategoryItem(const TSharedRef<SCustomDetailsView>& InCustomDetailsView
	, const TSharedPtr<ICustomDetailsViewItem>& InParentItem, FName InCategoryName, const FText& InLabel, const FText& InToolTip)
	: FCustomDetailsViewItemBase(InCustomDetailsView, InParentItem)
	, CategoryName(InCategoryName)
	, Label(InLabel)
	, ToolTip(InToolTip)
{
	NodeType = EDetailNodeType::Category;
}

void FCustomDetailsViewCustomCategoryItem::InitWidget_Internal()
{
	CreateNameWidget();
}

void FCustomDetailsViewCustomCategoryItem::CreateNameWidget()
{
	SetOverrideWidget(
		ECustomDetailsViewWidgetType::Name,
		SNew(STextBlock)
		.Text(Label)
		.ToolTipText(ToolTip)
		.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DetailsView.CategoryTextStyle"))
		.ShadowOffset(FVector2D::ZeroVector)
	);
}

void FCustomDetailsViewCustomCategoryItem::RefreshItemId()
{
	TSharedPtr<ICustomDetailsViewItem> Parent = ParentWeak.Pin();
	check(Parent.IsValid());
	ItemId = FCustomDetailsViewItemId::MakeCategoryId(CategoryName, &Parent->GetItemId());
}

void FCustomDetailsViewCustomCategoryItem::SetLabel(const FText& InLabel)
{
	Label = InLabel;
	CreateNameWidget();
}

void FCustomDetailsViewCustomCategoryItem::SetToolTip(const FText& InToolTip)
{
	ToolTip = InToolTip;
	CreateNameWidget();
}

TSharedRef<ICustomDetailsViewItem> FCustomDetailsViewCustomCategoryItem::AsItem()
{
	return SharedThis(this);
}

#undef LOCTEXT_NAMESPACE
