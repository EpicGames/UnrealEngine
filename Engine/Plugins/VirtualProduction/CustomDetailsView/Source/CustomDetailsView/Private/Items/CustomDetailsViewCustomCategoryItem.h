// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/CustomDetailsViewItemBase.h"
#include "Items/ICustomDetailsViewCustomCategoryItem.h"

class FText;

class FCustomDetailsViewCustomCategoryItem : public FCustomDetailsViewItemBase, public ICustomDetailsViewCustomCategoryItem
{
public:
	explicit FCustomDetailsViewCustomCategoryItem(const TSharedRef<SCustomDetailsView>& InCustomDetailsView
		, const TSharedPtr<ICustomDetailsViewItem>& InParentItem, FName InCategoryName
		, const FText& InLabel, const FText& InToolTip);

	virtual ~FCustomDetailsViewCustomCategoryItem() override = default;

	FName GetCategoryName() const { return CategoryName; }

	//~ Begin ICustomDetailsViewItem
	virtual void RefreshItemId() override;
	//~ End ICustomDetailsViewItem

	//~ Begin ICustomDetailsViewCustomItem
	virtual void SetLabel(const FText& InLabel) override;
	virtual void SetToolTip(const FText& InToolTip) override;
	virtual TSharedRef<ICustomDetailsViewItem> AsItem() override;
	//~ End ICustomDetailsViewCustomItem

protected:
	//~ Begin FCustomDetailsViewItemBase
	virtual void InitWidget_Internal() override;
	//~ End FCustomDetailsViewItemBase

	void CreateNameWidget();

	FName CategoryName;

	FText Label;

	FText ToolTip;
};
