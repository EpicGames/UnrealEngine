// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomDetailsViewCustomItem.h"
#include "Internationalization/Text.h"
#include "CustomDetailsViewItemBase.h"
#include "SCustomDetailsView.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CustomDetailsViewCustomItem"

FCustomDetailsViewCustomItem::FCustomDetailsViewCustomItem(const TSharedRef<SCustomDetailsView>& InCustomDetailsView
	, const TSharedPtr<ICustomDetailsViewItem>& InParentItem, FName InItemName, const FText& InLabel, const FText& InToolTip)
	: FCustomDetailsViewItemBase(InCustomDetailsView, InParentItem)
	, ItemName(InItemName)
	, Label(InLabel)
	, ToolTip(InToolTip)
{
}

void FCustomDetailsViewCustomItem::InitWidget_Internal()
{
	CreateNameWidget();
	SetOverrideWidget(ECustomDetailsViewWidgetType::Value, nullptr);
}

void FCustomDetailsViewCustomItem::CreateNameWidget()
{
	SetOverrideWidget(
		ECustomDetailsViewWidgetType::Name,
		SNew(STextBlock)
			.Text(Label)
			.ToolTipText(ToolTip)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.ShadowOffset(FVector2D(1.0f, 1.0f))
	);

	SetOverrideWidget(ECustomDetailsViewWidgetType::WholeRow, nullptr);
}

void FCustomDetailsViewCustomItem::RefreshItemId()
{
	TSharedPtr<ICustomDetailsViewItem> Parent = ParentWeak.Pin();
	check(Parent.IsValid());
	ItemId = FCustomDetailsViewItemId::MakeCustomId(ItemName, &Parent->GetItemId());
}

void FCustomDetailsViewCustomItem::SetNodeType(TOptional<EDetailNodeType> InNodeType)
{
	NodeType = InNodeType;
}

void FCustomDetailsViewCustomItem::SetLabel(const FText& InLabel)
{
	Label = InLabel;
	CreateNameWidget();
}

void FCustomDetailsViewCustomItem::SetToolTip(const FText& InToolTip)
{
	ToolTip = InToolTip;
	CreateNameWidget();
}

void FCustomDetailsViewCustomItem::SetValueWidget(const TSharedRef<SWidget>& InValueWidget)
{
	SetOverrideWidget(ECustomDetailsViewWidgetType::Value, InValueWidget);
	SetOverrideWidget(ECustomDetailsViewWidgetType::WholeRow, nullptr);
}

void FCustomDetailsViewCustomItem::SetExpansionWidget(const TSharedRef<SWidget>& InExpansionWidget)
{
	SetOverrideWidget(ECustomDetailsViewWidgetType::Extensions, InExpansionWidget);
	SetOverrideWidget(ECustomDetailsViewWidgetType::WholeRow, nullptr);
}

void FCustomDetailsViewCustomItem::SetWholeRowWidget(const TSharedRef<SWidget>& InWholeRowWidget)
{
	SetOverrideWidget(ECustomDetailsViewWidgetType::WholeRow, InWholeRowWidget);
	SetOverrideWidget(ECustomDetailsViewWidgetType::Name, nullptr);
	SetOverrideWidget(ECustomDetailsViewWidgetType::Value, nullptr);
	SetOverrideWidget(ECustomDetailsViewWidgetType::Extensions, nullptr);
}

TSharedRef<ICustomDetailsViewItem> FCustomDetailsViewCustomItem::AsItem()
{
	return SharedThis(this);
}

#undef LOCTEXT_NAMESPACE
