// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsView/Private/Items/CustomDetailsViewItemBase.h"
#include "Items/ICustomDetailsViewCustomItem.h"

class FText;

class FCustomDetailsViewCustomItem : public FCustomDetailsViewItemBase, public ICustomDetailsViewCustomItem
{
public:
	explicit FCustomDetailsViewCustomItem(const TSharedRef<SCustomDetailsView>& InCustomDetailsView
		, const TSharedPtr<ICustomDetailsViewItem>& InParentItem, FName InItemName
		, const FText& InLabel, const FText& InToolTip);

	virtual ~FCustomDetailsViewCustomItem() override = default;

	FName GetItemName() const { return ItemName; }

	//~ Begin ICustomDetailsViewItem
	virtual void RefreshItemId() override;
	//~ End ICustomDetailsViewItem

	//~ Begin ICustomDetailsViewCustomItem
	virtual void SetNodeType(TOptional<EDetailNodeType> InNodeType) override;
	virtual void SetLabel(const FText& InLabel) override;
	virtual void SetToolTip(const FText& InToolTip) override;
	virtual void SetValueWidget(const TSharedRef<SWidget>& InValueWidget) override;
	virtual void SetExpansionWidget(const TSharedRef<SWidget>& InExpansionWidget) override;
	virtual void SetWholeRowWidget(const TSharedRef<SWidget>& InWholeRowWidget) override;
	virtual TSharedRef<ICustomDetailsViewItem> AsItem() override;
	//~ End ICustomDetailsViewCustomItem

protected:
	//~ Begin FCustomDetailsViewItemBase
	virtual void InitWidget_Internal() override;
	//~ End FCustomDetailsViewItemBase

	void CreateNameWidget();

	FName ItemName;

	FText Label;

	FText ToolTip;
};
