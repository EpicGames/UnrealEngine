// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomDetailsViewDetailTreeNodeItem.h"

#include "CustomDetailsViewMenuContext.h"
#include "CustomDetailsViewSequencer.h"
#include "DetailColumnSizeData.h"
#include "DetailRowMenuContext.h"
#include "DetailTreeNode.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailKeyframeHandler.h"
#include "IDetailPropertyRow.h"
#include "IDetailTreeNode.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "SCustomDetailsView.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "CustomDetailsViewItem"

FCustomDetailsViewDetailTreeNodeItem::FCustomDetailsViewDetailTreeNodeItem(const TSharedRef<SCustomDetailsView>& InCustomDetailsView
		, const TSharedPtr<ICustomDetailsViewItem>& InParentItem
		, const TSharedPtr<IDetailTreeNode>& InDetailTreeNode)
	: FCustomDetailsViewItemBase(InCustomDetailsView, InParentItem)
	, DetailTreeNodeWeak(InDetailTreeNode)
{
}

FCustomDetailsViewDetailTreeNodeItem::~FCustomDetailsViewDetailTreeNodeItem()
{
	if (FSlateApplication::IsInitialized() && UpdateResetToDefaultHandle.IsValid())
	{
		FSlateApplication::Get().OnPostTick().Remove(UpdateResetToDefaultHandle);
	}
}

void FCustomDetailsViewDetailTreeNodeItem::RefreshItemId()
{
	TSharedPtr<ICustomDetailsViewItem> Parent = ParentWeak.Pin();
	check(Parent.IsValid());

	if (TSharedPtr<IDetailTreeNode> DetailTreeNode = DetailTreeNodeWeak.Pin())
	{
		ItemId = FCustomDetailsViewItemId::MakeFromDetailTreeNode(DetailTreeNode.ToSharedRef(), &Parent->GetItemId());
	}
	else
	{
		ItemId = FCustomDetailsViewItemId();
	}
}

void FCustomDetailsViewDetailTreeNodeItem::InitWidget_Internal()
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	TSharedPtr<IDetailTreeNode> DetailTreeNode = DetailTreeNodeWeak.Pin();

	if (!DetailTreeNode)
	{
		return;
	}

	PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	NodeType = DetailTreeNode->GetNodeType();

	const FDetailTreeNode* const DetailTreeNodePtr = static_cast<FDetailTreeNode*>(DetailTreeNode.Get());
	DetailTreeNodePtr->GenerateStandaloneWidget(DetailWidgetRow);

	const TAttribute<bool> CanEditPropertyAttribute = PropertyHandle.IsValid()
		? DetailTreeNodePtr->IsPropertyEditingEnabled()
		: TAttribute<bool>();

	const TAttribute<bool> EditConditionAttribute   = DetailWidgetRow.EditConditionValue;
	const TAttribute<bool> RowEnabledAttribute      = DetailWidgetRow.IsEnabledAttr;
	const TAttribute<bool> RowValueEnabledAttribute = DetailWidgetRow.IsValueEnabledAttr;

	const TAttribute<bool> IsEnabledAttribute = TAttribute<bool>::CreateLambda(
		[CanEditPropertyAttribute, RowEnabledAttribute, EditConditionAttribute]()
		{
			return CanEditPropertyAttribute.Get(true)
				&& RowEnabledAttribute.Get(true)
				&& EditConditionAttribute.Get(true);
		});

	const TAttribute<bool> IsValueEnabledAttribute = TAttribute<bool>::CreateLambda(
		[IsEnabledAttribute, RowValueEnabledAttribute]()
		{
			return IsEnabledAttribute.Get()
				&& RowValueEnabledAttribute.Get(true);
		});

	DetailWidgetRow.NameWidget.Widget->SetEnabled(IsEnabledAttribute);
	DetailWidgetRow.ValueWidget.Widget->SetEnabled(IsValueEnabledAttribute);
	DetailWidgetRow.ExtensionWidget.Widget->SetEnabled(IsEnabledAttribute);
}

TSharedPtr<IDetailsView> FCustomDetailsViewDetailTreeNodeItem::GetDetailsView() const
{
	if (const TSharedPtr<IDetailTreeNode> DetailTreeNode = GetRowTreeNode())
	{
		return DetailTreeNode->GetNodeDetailsViewSharedPtr();
	}

	TSharedPtr<ICustomDetailsViewItem> ParentItem = GetParent();

	while (ParentItem.IsValid())
	{
		if (TSharedPtr<IDetailsView> DetailsView = ParentItem->GetDetailsView())
		{
			return DetailsView;
		}

		ParentItem = ParentItem->GetParent();
	}

	return nullptr;
}

void FCustomDetailsViewDetailTreeNodeItem::SetResetToDefaultOverride(const FResetToDefaultOverride& InOverride)
{
	DetailWidgetRow.CustomResetToDefault = InOverride;
}

void FCustomDetailsViewDetailTreeNodeItem::CreateGlobalExtensionButtons(TArray<FPropertyRowExtensionButton>& OutExtensionButtons)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FOnGenerateGlobalRowExtensionArgs RowExtensionArgs;
	RowExtensionArgs.OwnerTreeNode = DetailTreeNodeWeak;
	RowExtensionArgs.PropertyHandle = PropertyHandle;
	PropertyEditorModule.GetGlobalRowExtensionDelegate().Broadcast(RowExtensionArgs, OutExtensionButtons);
}

bool FCustomDetailsViewDetailTreeNodeItem::CreateResetToDefaultButton(FPropertyRowExtensionButton& OutButton)
{
	OutButton.Label = LOCTEXT("ResetToDefault", "Reset to Default");
	OutButton.ToolTip = TAttribute<FText>::CreateSP(this, &FCustomDetailsViewDetailTreeNodeItem::GetResetToDefaultToolTip);
	OutButton.Icon = TAttribute<FSlateIcon>::CreateSP(this, &FCustomDetailsViewDetailTreeNodeItem::GetResetToDefaultIcon);

	OutButton.UIAction = FUIAction(FExecuteAction::CreateSP(this, &FCustomDetailsViewDetailTreeNodeItem::OnResetToDefaultClicked)
		, FCanExecuteAction::CreateSP(this, &FCustomDetailsViewDetailTreeNodeItem::CanResetToDefault));

	// Add Updating Reset To Default to the Slate App PostTick
	if (!UpdateResetToDefaultHandle.IsValid())
	{
		UpdateResetToDefaultHandle = FSlateApplication::Get().OnPostTick().AddSP(this, &FCustomDetailsViewDetailTreeNodeItem::UpdateResetToDefault);
	}

	return true;
}

void FCustomDetailsViewDetailTreeNodeItem::SetCreateChildItemDelegate(FOnCustomDetailsViewGenerateChildItem InDelegate)
{
	ChildItemDelegate = InDelegate;
}

void FCustomDetailsViewDetailTreeNodeItem::SetCustomizeItemMenuContext(FOnCustomDetailsViewCustomizeItemMenuContext InDelegate)
{
	ContextMenuDelegate = InDelegate;
}

void FCustomDetailsViewDetailTreeNodeItem::AddExtensionWidget(const TSharedRef<SSplitter>& InSplitter, const FDetailColumnSizeData& InColumnSizeData, const FCustomDetailsViewArgs& InViewArgs)
{
	if (GetOverrideWidget(ECustomDetailsViewWidgetType::Extensions).IsValid())
	{
		FCustomDetailsViewItemBase::AddExtensionWidget(InSplitter, InColumnSizeData, InViewArgs);
		return;
	}

	TArray<FPropertyRowExtensionButton> ExtensionButtons;

	// Reset to Default
	if (InViewArgs.bAllowResetToDefault)
	{
		FPropertyRowExtensionButton ResetToDefaultButton;

		if (CreateResetToDefaultButton(ResetToDefaultButton))
		{
			ExtensionButtons.Add(MoveTemp(ResetToDefaultButton));
		}
	}

	// Global Extensions
	if (InViewArgs.bAllowGlobalExtensions)
	{
		CreateGlobalExtensionButtons(ExtensionButtons);

		// Sequencer relies on getting the Keyframe Handler via the Details View of the IDetailTreeNode, but is null since there's no Details View here
		// instead add it manually here

		if (bKeyframeEnabled)
		{
			FCustomDetailsViewSequencerUtils::CreateSequencerExtensionButton(InViewArgs.KeyframeHandler, PropertyHandle, ExtensionButtons);
		}
	}

	if (ExtensionButtons.IsEmpty())
	{
		return;
	}

	TSharedRef<SWidget> ExtensionWidget = CreateExtensionButtonWidget(ExtensionButtons);

	Widgets.Add(ECustomDetailsViewWidgetType::Extensions, ExtensionWidget);

	InSplitter->AddSlot()
		.Value(InColumnSizeData.GetRightColumnWidth())
		.MinSize(InColumnSizeData.GetRightColumnMinWidth())
		.OnSlotResized(InColumnSizeData.GetOnRightColumnResized())
		[
			ExtensionWidget
		];
}

TSharedRef<SWidget> FCustomDetailsViewDetailTreeNodeItem::MakeEditConditionWidget()
{
	return SNew(SCheckBox)
		.OnCheckStateChanged(this, &FCustomDetailsViewDetailTreeNodeItem::OnEditConditionCheckChanged)
		.IsChecked(this, &FCustomDetailsViewDetailTreeNodeItem::GetEditConditionCheckState)
		.Visibility(this, &FCustomDetailsViewDetailTreeNodeItem::GetEditConditionVisibility);
}

bool FCustomDetailsViewDetailTreeNodeItem::HasEditConditionToggle() const
{
	return DetailWidgetRow.OnEditConditionValueChanged.IsBound();
}

EVisibility FCustomDetailsViewDetailTreeNodeItem::GetEditConditionVisibility() const
{
	return HasEditConditionToggle() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState FCustomDetailsViewDetailTreeNodeItem::GetEditConditionCheckState() const
{
	return DetailWidgetRow.EditConditionValue.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FCustomDetailsViewDetailTreeNodeItem::OnEditConditionCheckChanged(ECheckBoxState InCheckState)
{
	checkSlow(HasEditConditionToggle());
	FScopedTransaction EditConditionChangedTransaction(LOCTEXT("EditConditionChanged", "Edit Condition Changed"));
	DetailWidgetRow.OnEditConditionValueChanged.ExecuteIfBound(InCheckState == ECheckBoxState::Checked);
}

void FCustomDetailsViewDetailTreeNodeItem::OnKeyframeClicked()
{
	const TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = GetKeyframeHandler();
	if (KeyframeHandler.IsValid() && PropertyHandle.IsValid())
	{
		KeyframeHandler->OnKeyPropertyClicked(*PropertyHandle);
	}
}

bool FCustomDetailsViewDetailTreeNodeItem::IsKeyframeVisible() const
{
	const TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = GetKeyframeHandler();

	if (!KeyframeHandler.IsValid() || !PropertyHandle.IsValid())
	{
		return false;
	}

	const UClass* const ObjectClass = PropertyHandle->GetOuterBaseClass();
	if (ObjectClass == nullptr)
	{
		return false;
	}

	return KeyframeHandler->IsPropertyKeyable(ObjectClass, *PropertyHandle);
}

bool FCustomDetailsViewDetailTreeNodeItem::IsResetToDefaultVisible() const
{
	return bResetToDefaultVisible;
}

void FCustomDetailsViewDetailTreeNodeItem::UpdateResetToDefault(float InDeltaTime)
{
	bResetToDefaultVisible = false;

	if (DetailWidgetRow.CustomResetToDefault.IsSet())
	{
		bResetToDefaultVisible = DetailWidgetRow.CustomResetToDefault.GetValue().IsResetToDefaultVisible(PropertyHandle);
		return;
	}

	if (PropertyHandle.IsValid())
	{
		if (PropertyHandle->HasMetaData("NoResetToDefault") || PropertyHandle->GetInstanceMetaData("NoResetToDefault"))
		{
			bResetToDefaultVisible = false;
			return;
		}
		bResetToDefaultVisible = PropertyHandle->CanResetToDefault();
	}
}

bool FCustomDetailsViewDetailTreeNodeItem::CanResetToDefault() const
{
	return IsResetToDefaultVisible() && DetailWidgetRow.ValueWidget.Widget->IsEnabled();
}

void FCustomDetailsViewDetailTreeNodeItem::OnResetToDefaultClicked()
{
	if (DetailWidgetRow.CustomResetToDefault.IsSet())
	{
		DetailWidgetRow.CustomResetToDefault.GetValue().OnResetToDefaultClicked(PropertyHandle);
	}
	else if (PropertyHandle.IsValid())
	{
		PropertyHandle->ResetToDefault();
	}
}

FText FCustomDetailsViewDetailTreeNodeItem::GetResetToDefaultToolTip() const
{
	return IsResetToDefaultVisible()
		? LOCTEXT("ResetToDefaultPropertyValueToolTip", "Reset this property to its default value.")
		: FText::GetEmpty();
}

FSlateIcon FCustomDetailsViewDetailTreeNodeItem::GetResetToDefaultIcon() const
{
	static const FSlateIcon ResetIcon_Enabled(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault");
	static const FSlateIcon ResetIcon_Disabled(FAppStyle::Get().GetStyleSetName(), "NoBrush");

	return IsResetToDefaultVisible()
		? ResetIcon_Enabled
		: ResetIcon_Disabled;
}

TSharedPtr<SWidget> FCustomDetailsViewDetailTreeNodeItem::GenerateContextMenuWidget()
{
	UToolMenus* Menus = UToolMenus::Get();

	check(Menus);

	static const FName DetailViewContextMenuName = UE::PropertyEditor::RowContextMenuName;

	if (!Menus->IsMenuRegistered(DetailViewContextMenuName))
	{
		return nullptr;
	}

	const TSharedPtr<IPropertyHandle> RowPropertyHandle = GetRowPropertyHandle();

	if (!RowPropertyHandle.IsValid())
	{
		return nullptr;
	}

	UDetailRowMenuContext* RowMenuContext = NewObject<UDetailRowMenuContext>();
	RowMenuContext->PropertyHandles.Add(RowPropertyHandle);
	RowMenuContext->DetailsView = GetDetailsView();
	RowMenuContext->ForceRefreshWidget().AddSPLambda(this, [this]
	{
		RefreshChildren();
	});

	// Customize context for this menu
	if (const TSharedPtr<ICustomDetailsView> DetailsView = CustomDetailsViewWeak.Pin())
	{
		ContextMenuDelegate.ExecuteIfBound(DetailsView.ToSharedRef(), AsShared(), RowMenuContext, RowMenuContext->PropertyHandles);
	}

	FToolMenuContext ToolMenuContext(RowMenuContext);
	ToolMenuContext.AddObject(GetMutableDefault<UCustomDetailsViewMenuContext>());

	return Menus->GenerateWidget(DetailViewContextMenuName, ToolMenuContext);
}

void FCustomDetailsViewDetailTreeNodeItem::GenerateCustomChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren)
{
	if (!CustomDetailsViewWeak.IsValid())
	{
		return;
	}

	const TSharedPtr<IDetailTreeNode> DetailTreeNode = DetailTreeNodeWeak.Pin();

	if (!DetailTreeNode.IsValid())
	{
		return;
	}

	const ECustomDetailsViewNodePropertyFlag ChildNodePropertyFlags = (IsStruct() || HasParentStruct())
		? ECustomDetailsViewNodePropertyFlag::HasParentStruct
		: ECustomDetailsViewNodePropertyFlag::None;

	TArray<TSharedRef<IDetailTreeNode>> NodeChildren;
	DetailTreeNode->GetChildren(NodeChildren, /* Ignore Visibility */ true);

	AddChildDetailsTreeNodes(InParentItem, ChildNodePropertyFlags, NodeChildren, OutChildren);
}

void FCustomDetailsViewDetailTreeNodeItem::AddChildDetailsTreeNodes(const TSharedRef<ICustomDetailsViewItem>& InParentItem, ECustomDetailsViewNodePropertyFlag InNodeChildPropertyFlag,
	const TArray<TSharedRef<IDetailTreeNode>>& InNodeChildren, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren)
{
	if (!CustomDetailsViewWeak.IsValid())
	{
		return;
	}

	const TSharedRef<SCustomDetailsView> CustomDetailsView = CustomDetailsViewWeak.Pin().ToSharedRef();

	for (const TSharedRef<IDetailTreeNode>& ChildTreeNode : InNodeChildren)
	{
		using namespace UE::CustomDetailsView::Private;
		const EAllowType AllowType = CustomDetailsView->GetAllowType(InParentItem, ChildTreeNode, InNodeChildPropertyFlag);

		// If DisallowSelfAndChildren, this Tree Node Path is completely blocked, continue.
		if (AllowType == EAllowType::DisallowSelfAndChildren)
		{
			continue;
		}

		// If DisallowSelf, grab the children nodes. Self's Children node's parent is set to Self's Parent rather than Self
		if (AllowType == EAllowType::DisallowSelf)
		{
			if (ChildItemDelegate.IsBound())
			{
				if (TSharedPtr<ICustomDetailsViewItem> CustomChildItem = ChildItemDelegate.Execute(CustomDetailsView, InParentItem, ChildTreeNode))
				{
					CustomChildItem->RefreshItemId();
					CustomChildItem->RefreshChildren(InParentItem);

					OutChildren.Append(CustomChildItem->GetChildren());

					continue;
				}
			}

			FCustomDetailsViewDetailTreeNodeItem ChildItem = FCustomDetailsViewDetailTreeNodeItem(CustomDetailsView, InParentItem, ChildTreeNode);
			ChildItem.RefreshItemId();
			ChildItem.RefreshChildren(InParentItem);

			OutChildren.Append(ChildItem.GetChildren());

			continue;
		}

		// Support Type here has to be allowed
		check(AllowType == EAllowType::Allowed);

		if (ChildItemDelegate.IsBound())
		{
			if (TSharedPtr<ICustomDetailsViewItem> CustomChildItem = ChildItemDelegate.Execute(CustomDetailsView, InParentItem, ChildTreeNode))
			{
				CustomChildItem->AddAsChild(InParentItem, OutChildren);
				continue;
			}
		}

		TSharedRef<FCustomDetailsViewDetailTreeNodeItem> Item = CustomDetailsView->CreateItem<FCustomDetailsViewDetailTreeNodeItem>(CustomDetailsView
			, InParentItem, ChildTreeNode);

		Item->AddAsChild(InParentItem, OutChildren);
	}
}

bool FCustomDetailsViewDetailTreeNodeItem::IsStruct() const
{
	if (PropertyHandle.IsValid())
	{
		if (FProperty* Property = PropertyHandle->GetProperty())
		{
			return Property->IsA<FStructProperty>();
		}
	}

	return false;
}

bool FCustomDetailsViewDetailTreeNodeItem::HasParentStruct() const
{
	for (TSharedPtr<ICustomDetailsViewItem> Parent = GetParent(); Parent.IsValid(); Parent = Parent->GetParent())
	{
		if (Parent->GetItemId().IsType(EDetailNodeType::Item))
		{
			if (StaticCastSharedPtr<FCustomDetailsViewDetailTreeNodeItem>(Parent)->IsStruct())
			{
				return true;
			}
		}
	}

	return false;
}

void FCustomDetailsViewDetailTreeNodeItem::UpdateVisibility()
{
	if (!PropertyHandle.IsValid() || !PropertyHandle->HasMetaData(TEXT("EditConditionHides")))
	{
		FCustomDetailsViewItemBase::UpdateVisibility();
		return;
	}

	DetailWidgetRow.VisibilityAttr = TAttribute<EVisibility>::CreateLambda(
		[ParentWeak = ParentWeak, EditConditionAttribute = DetailWidgetRow.EditConditionValue, OriginalAttr = DetailWidgetRow.VisibilityAttr]()
		{
			if (!EditConditionAttribute.Get(true))
			{
				return EVisibility::Collapsed;
			}

			if (OriginalAttr.Get(EVisibility::Visible) != EVisibility::Visible)
			{
				return EVisibility::Collapsed;
			}

			if (TSharedPtr<ICustomDetailsViewItem> Parent = ParentWeak.Pin())
			{
				if (Parent->GetDetailWidgetRow().VisibilityAttr.Get(EVisibility::Visible) != EVisibility::Visible)
				{
					return EVisibility::Collapsed;
				}
			}

			return EVisibility::Visible;
		}
	);		
}


#undef LOCTEXT_NAMESPACE
