// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsView/Private/Items/CustomDetailsViewItemBase.h"
#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "Layout/Visibility.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"

class FCustomDetailsViewRootItem;
class FDetailColumnSizeData;
class ICustomDetailsView;
class IDetailTreeNode;
class IPropertyRowGenerator;
class SSplitter;
enum class EDetailNodeType;
struct FCustomDetailsViewArgs;

class FCustomDetailsViewDetailTreeNodeItem : public FCustomDetailsViewItemBase
{
public:
	explicit FCustomDetailsViewDetailTreeNodeItem(const TSharedRef<SCustomDetailsView>& InCustomDetailsView
		, const TSharedPtr<ICustomDetailsViewItem>& InParentItem
		, const TSharedPtr<IDetailTreeNode>& InDetailTreeNode);

	virtual ~FCustomDetailsViewDetailTreeNodeItem() override;

	
	//~ Begin ICustomDetailsViewItem
	virtual TSharedPtr<IDetailsView> GetDetailsView() const override;
	virtual void RefreshItemId() override;
	virtual const TArray<TSharedPtr<ICustomDetailsViewItem>>& GetChildren() const override final { return Children; }
	virtual void SetResetToDefaultOverride(const FResetToDefaultOverride& InOverride) override;
	virtual void CreateGlobalExtensionButtons(TArray<FPropertyRowExtensionButton>& OutExtensionButtons) override;
	virtual bool CreateResetToDefaultButton(FPropertyRowExtensionButton& OutButton) override;
	virtual void SetCreateChildItemDelegate(FOnCustomDetailsViewGenerateChildItem InDelegate) override;
	virtual void SetCustomizeItemMenuContext(FOnCustomDetailsViewCustomizeItemMenuContext InDelegate) override;
	//~ End ICustomDetailsViewItem

	//~ Begin FCustomDetailsViewItemBase
	virtual void AddExtensionWidget(const TSharedRef<SSplitter>& InSplitter
		, const FDetailColumnSizeData& InColumnSizeData
		, const FCustomDetailsViewArgs& InViewArgs) override;

	virtual TSharedRef<SWidget> MakeEditConditionWidget() override;
	//~ End FCustomDetailsViewItemBase

	bool HasEditConditionToggle() const;
	EVisibility GetEditConditionVisibility() const;
	ECheckBoxState GetEditConditionCheckState() const;
	void OnEditConditionCheckChanged(ECheckBoxState InCheckState);

	void OnKeyframeClicked();
	bool IsKeyframeVisible() const;

	bool IsResetToDefaultVisible() const;
	void UpdateResetToDefault(float InDeltaTime);

	void OnResetToDefaultClicked();
	bool CanResetToDefault() const;

	FText GetResetToDefaultToolTip() const;
	FSlateIcon GetResetToDefaultIcon() const;

	TSharedPtr<IDetailTreeNode> GetRowTreeNode() const
	{
		return DetailTreeNodeWeak.Pin();
	}

	TSharedPtr<IPropertyHandle> GetRowPropertyHandle() const
	{
		return PropertyHandle;
	}

protected:
	void AddChildDetailsTreeNodes(const TSharedRef<ICustomDetailsViewItem>& InParentItem, ECustomDetailsViewNodePropertyFlag InNodeChildPropertyFlag, 
		const TArray<TSharedRef<IDetailTreeNode>>& InNodeChildren, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren);

	bool IsStruct() const;

	bool HasParentStruct() const;

	//~ Begin FCustomDetailsViewItemBase
	/** Generate details context menu based on property handle */
	virtual void InitWidget_Internal() override;
	virtual void UpdateVisibility() override;
	virtual TSharedPtr<SWidget> GenerateContextMenuWidget() override;
	virtual void GenerateCustomChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) override;
	//~ End FCustomDetailsViewItemBase

	/** The Property Handle of this Detail Tree Node. Can be null */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Weak pointer to the Detail Tree Node this Item represents */
	TWeakPtr<IDetailTreeNode> DetailTreeNodeWeak;

	/** Handle to the delegate handling Updating Reset To Default Visibility State Post Slate App Tick */
	FDelegateHandle UpdateResetToDefaultHandle;

	/** Cached value of the visibility state of the ResetToDefault Widget */
	bool bResetToDefaultVisible = false;

	/** Used for custom child property rows. */
	FOnCustomDetailsViewGenerateChildItem ChildItemDelegate;

	/** Used to customize the context menu */
	FOnCustomDetailsViewCustomizeItemMenuContext ContextMenuDelegate;
};
