// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/CustomDetailsViewItemBase.h"
#include "DetailColumnSizeData.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailKeyframeHandler.h"
#include "IDetailTreeNode.h"
#include "SCustomDetailsView.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBox.h"

namespace UE::CustomDetailsView::Private
{
	TAttribute<FOptionalSize> GetOptionalSize(const TOptional<float>& InOptional)
	{
		if (InOptional.IsSet())
		{
			return FOptionalSize(*InOptional);
		}
		return TAttribute<FOptionalSize>();
	}

	TAttribute<FSlateColor> GetBackgroundColorAttribute(TOptional<EDetailNodeType> InNodeType
		, int32 InIndentLevel
		, TAttribute<bool> IsHoveredAttribute
		, float InBackgroundOpacity)
	{
		if (InNodeType.IsSet() && *InNodeType == EDetailNodeType::Item)
		{
			static const TArray<int32, TInlineAllocator<5>> Offsets{0, 4, 8, 12};
			static const int32 OffsetNum = Offsets.Num();

			int32 ColorIndex = InIndentLevel % OffsetNum;

			// As Indent Level continuously Color Index should go up and down in a Ping Pong way
			if ((InIndentLevel / OffsetNum) % 2 != 0)
			{
				ColorIndex = OffsetNum - 1 - ColorIndex;
			}
			uint8 Offset = Offsets[ColorIndex];

			return TAttribute<FSlateColor>::CreateLambda([Offset, IsHoveredAttribute, InBackgroundOpacity]()
				{
					static const FColor HeaderColor = FAppStyle::Get().GetSlateColor("Colors.Header").GetSpecifiedColor().ToFColor(true);
					static const FColor PanelColor = FAppStyle::Get().GetSlateColor("Colors.Panel").GetSpecifiedColor().ToFColor(true);

					const FColor& BaseColor = IsHoveredAttribute.Get(false) ? HeaderColor : PanelColor;

					const FColor ColorWithOffset(BaseColor.R + Offset
						, BaseColor.G + Offset
						, BaseColor.B + Offset);

					FLinearColor Color = FLinearColor::FromSRGBColor(ColorWithOffset);
					Color.A = InBackgroundOpacity;
					return FSlateColor(Color);
				});
		}

		return TAttribute<FSlateColor>();
	}
}

FCustomDetailsViewItemBase::FCustomDetailsViewItemBase(const TSharedRef<SCustomDetailsView>& InCustomDetailsView,
	const TSharedPtr<ICustomDetailsViewItem>& InParentItem)
	: CustomDetailsViewWeak(InCustomDetailsView)
	, ParentWeak(InParentItem)
	, NodeType(EDetailNodeType::Item)
{
}

void FCustomDetailsViewItemBase::UpdateIndentLevel()
{
	IndentLevel = -1;
	TSharedPtr<ICustomDetailsViewItem> Parent = ParentWeak.Pin();
	while (Parent.IsValid())
	{
		++IndentLevel;
		Parent = Parent->GetParent();
	}
}

void FCustomDetailsViewItemBase::InitWidget()
{
	UpdateIndentLevel();
	InitWidget_Internal();
	UpdateVisibility();
	RefreshItemId();
}

TArray<TSharedPtr<ICustomDetailsViewItem>> FCustomDetailsViewItemBase::GenerateChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem)
{
	const TSharedPtr<SCustomDetailsView> CustomDetailsView = CustomDetailsViewWeak.Pin();

	if (!CustomDetailsView.IsValid())
	{
		return {};
	}

	const UE::CustomDetailsView::FTreeExtensionType& TreeExtensions = CustomDetailsView->GetTreeExtensions(ItemId);

	TArray<TSharedPtr<ICustomDetailsViewItem>> OutChildren;

	GatherChildren(InParentItem, TreeExtensions, ECustomDetailsTreeInsertPosition::FirstChild, OutChildren);

	GenerateCustomChildren(InParentItem, OutChildren);

	GatherChildren(InParentItem, TreeExtensions, ECustomDetailsTreeInsertPosition::Child, OutChildren);
	GatherChildren(InParentItem, TreeExtensions, ECustomDetailsTreeInsertPosition::LastChild, OutChildren);

	return OutChildren;
}

void FCustomDetailsViewItemBase::UpdateVisibility()
{
	DetailWidgetRow.VisibilityAttr = TAttribute<EVisibility>::CreateLambda(
		[ParentWeak = ParentWeak, OriginalAttr = DetailWidgetRow.VisibilityAttr]()
		{
			if (OriginalAttr.Get(EVisibility::Visible) != EVisibility::Visible)
			{
				return EVisibility::Collapsed;
			}

			if (TSharedPtr<ICustomDetailsViewItem> Parent = ParentWeak.Pin())
			{
				if (Parent->GetDetailWidgetRow().VisibilityAttr.Get(EVisibility::Visible) != EVisibility::Visible)
				{
					return EVisibility::Collapsed;
				}
			}

			return EVisibility::Visible;
		}
	);
}

void FCustomDetailsViewItemBase::GatherChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem, 
	const UE::CustomDetailsView::FTreeExtensionType& InTreeExtensions, ECustomDetailsTreeInsertPosition InPosition, 
	TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren)
{
	if (const TArray<TSharedPtr<ICustomDetailsViewItem>>* ExtensionList = InTreeExtensions.Find(InPosition))
	{
		for (const TSharedPtr<ICustomDetailsViewItem>& Extension : *ExtensionList)
		{
			Extension->AddAsChild(InParentItem, OutChildren);
		}
	}
}

void FCustomDetailsViewItemBase::AddAsChild(const TSharedRef<ICustomDetailsViewItem>& InParentItem, 
	TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren)
{
	const TSharedPtr<SCustomDetailsView> CustomDetailsView = CustomDetailsViewWeak.Pin();

	if (!CustomDetailsView.IsValid())
	{
		return;
	}

	SetParent(InParentItem);
	RefreshItemId();
	RefreshChildren(SharedThis(this));

	const UE::CustomDetailsView::FTreeExtensionType& TreeExtensions = CustomDetailsView->GetTreeExtensions(ItemId);
	GatherChildren(InParentItem, TreeExtensions, ECustomDetailsTreeInsertPosition::Before, OutChildren);

	OutChildren.Add(SharedThis(this));

	GatherChildren(InParentItem, TreeExtensions, ECustomDetailsTreeInsertPosition::After, OutChildren);
}

TSharedPtr<ICustomDetailsView> FCustomDetailsViewItemBase::GetCustomDetailsView() const
{
	return CustomDetailsViewWeak.Pin();
}

const FCustomDetailsViewItemId& FCustomDetailsViewItemBase::GetItemId() const
{
	return ItemId;
}

TSharedPtr<ICustomDetailsViewItem> FCustomDetailsViewItemBase::GetRoot() const
{
	const TSharedPtr<ICustomDetailsViewItem> Parent = ParentWeak.Pin();

	if (!Parent.IsValid())
	{
		return SharedThis(const_cast<FCustomDetailsViewItemBase*>(this));
	}

	return Parent->GetRoot();
}

TSharedPtr<ICustomDetailsViewItem> FCustomDetailsViewItemBase::GetParent() const
{
	return ParentWeak.Pin();
}

void FCustomDetailsViewItemBase::SetParent(TSharedPtr<ICustomDetailsViewItem> InParent)
{
	ParentWeak = InParent;
}

const TArray<TSharedPtr<ICustomDetailsViewItem>>& FCustomDetailsViewItemBase::GetChildren() const
{
	return Children;
}

void FCustomDetailsViewItemBase::RefreshChildren(TSharedPtr<ICustomDetailsViewItem> InParentOverride)
{
	Children.Reset();

	if (!InParentOverride.IsValid())
	{
		InParentOverride = SharedThis(this);
	}

	Children = GenerateChildren(InParentOverride.ToSharedRef());
}

TOptional<EDetailNodeType> FCustomDetailsViewItemBase::GetNodeType() const
{
	return NodeType;
}

TSharedRef<SWidget> FCustomDetailsViewItemBase::MakeWidget(const TSharedPtr<SWidget>& InPrependWidget, const TSharedPtr<SWidget>& InOwningWidget)
{
	Widgets.Reset();

	TSharedPtr<SCustomDetailsView> CustomDetailsView = CustomDetailsViewWeak.Pin();
	check(CustomDetailsView.IsValid());

	const FCustomDetailsViewArgs& ViewArgs = CustomDetailsView->GetViewArgs();

	check(ViewArgs.ColumnSizeData.IsValid());
	const FDetailColumnSizeData& ColumnSizeData = *ViewArgs.ColumnSizeData;

	TSharedRef<SSplitter> Splitter = SNew(SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.PhysicalSplitterHandleSize(1.0f)
		.HitDetectionSplitterHandleSize(5.0f)
		.HighlightedHandleIndex(ColumnSizeData.GetHoveredSplitterIndex())
		.Orientation(Orient_Horizontal);

	const bool bWholeRowAllowed = ViewArgs.WidgetTypeAllowList.IsAllowed(ECustomDetailsViewWidgetType::WholeRow);
	const bool bNameAllowed = ViewArgs.WidgetTypeAllowList.IsAllowed(ECustomDetailsViewWidgetType::Name);
	const bool bValueAllowed = ViewArgs.WidgetTypeAllowList.IsAllowed(ECustomDetailsViewWidgetType::Value);

	const bool bExtensionsAllowed = (ViewArgs.bAllowResetToDefault || ViewArgs.bAllowGlobalExtensions)
		&& ViewArgs.WidgetTypeAllowList.IsAllowed(ECustomDetailsViewWidgetType::Extensions);

	const bool bHasWholeRow = bWholeRowAllowed
		&& (OverrideWidgets.Contains(ECustomDetailsViewWidgetType::WholeRow)
			|| (DetailWidgetRow.HasAnyContent() && !DetailWidgetRow.HasNameContent() && !DetailWidgetRow.HasValueContent()));

	const TSharedRef<SWidget>* OverrideNameWidget = OverrideWidgets.Find(ECustomDetailsViewWidgetType::Name);
	const TSharedRef<SWidget>* OverrideValueWidget = OverrideWidgets.Find(ECustomDetailsViewWidgetType::Value);

	const bool bHasName = bNameAllowed && ((OverrideNameWidget && (*OverrideNameWidget) != SNullWidget::NullWidget) || DetailWidgetRow.HasNameContent());
	const bool bHasValue = bValueAllowed && ((OverrideValueWidget && (*OverrideValueWidget) != SNullWidget::NullWidget) || DetailWidgetRow.HasValueContent());

	if (bHasWholeRow)
	{
		AddWholeRowWidget(Splitter, InPrependWidget, ColumnSizeData, ViewArgs.DefaultPadding);

		if (bExtensionsAllowed)
		{
			AddExtensionWidget(Splitter, ColumnSizeData, CustomDetailsView->GetViewArgs());
		}
	}
	else if (bHasName || bHasValue)
	{
		if (bHasName)
		{
			AddNameWidget(Splitter, InPrependWidget, ColumnSizeData, ViewArgs.DefaultPadding);
		}

		if (bHasValue)
		{
			AddValueWidget(Splitter, ColumnSizeData, ViewArgs.DefaultPadding);

			if (bExtensionsAllowed)
			{
				AddExtensionWidget(Splitter, ColumnSizeData, CustomDetailsView->GetViewArgs());
			}
		}
	}
	else
	{
		return SNullWidget::NullWidget;
	}

	TAttribute<bool> IsHoveredAttribute;
	if (InOwningWidget.IsValid())
	{
		IsHoveredAttribute = TAttribute<bool>(InOwningWidget.ToSharedRef(), &SWidget::IsHovered);
	}

	const FSlateBrush* NodeBrush = FAppStyle::GetBrush("DetailsView.CategoryMiddle");
	if (NodeType.IsSet() && *NodeType == EDetailNodeType::Category)
	{
		NodeBrush = FAppStyle::GetBrush("DetailsView.CategoryTop");
	}

	const TAttribute<FSlateColor> BackgroundColorAttribute = UE::CustomDetailsView::Private::GetBackgroundColorAttribute(
		NodeType
		, FMath::Max(IndentLevel - 1, 0)
		, IsHoveredAttribute
		, ViewArgs.RowBackgroundOpacity);

	TSharedRef<SWidget> ItemWidget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("DetailsView.GridLine"))
		.BorderBackgroundColor(FLinearColor(1.f, 1.f, 1.f, ViewArgs.RowBackgroundOpacity))
		.Padding(FMargin(0, 0, 0, 1))
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(SBox)
			.MinDesiredHeight(26.f)
			[
				SNew(SBorder)
				.BorderImage(NodeBrush)
				.BorderBackgroundColor(BackgroundColorAttribute)
				.Padding(0)
				.OnMouseButtonDown(this, &FCustomDetailsViewItemBase::OnMouseButtonDown)
				[
					Splitter
				]
			]
		];

	ViewArgs.OnItemWidgetGenerated.Broadcast(SharedThis(this));
	return ItemWidget;
}

TSharedPtr<SWidget> FCustomDetailsViewItemBase::GetWidget(ECustomDetailsViewWidgetType InWidgetType) const
{
	if (const TSharedPtr<SWidget>* const FoundWidget = Widgets.Find(InWidgetType))
	{
		return *FoundWidget;
	}
	return nullptr;
}

TSharedPtr<SWidget> FCustomDetailsViewItemBase::GetOverrideWidget(ECustomDetailsViewWidgetType InWidgetType) const
{
	if (const TSharedRef<SWidget>* const FoundWidget = OverrideWidgets.Find(InWidgetType))
	{
		return *FoundWidget;
	}
	return nullptr;
}

void FCustomDetailsViewItemBase::SetOverrideWidget(ECustomDetailsViewWidgetType InWidgetType, TSharedPtr<SWidget> InWidget)
{
	if (InWidget == nullptr || InWidget == SNullWidget::NullWidget)
	{
		if (OverrideWidgets.Contains(InWidgetType))
		{
			OverrideWidgets.Remove(InWidgetType);
		}

		return;
	}

	OverrideWidgets.Add(InWidgetType, InWidget.ToSharedRef());
}

void FCustomDetailsViewItemBase::SetKeyframeEnabled(bool bInKeyframeEnabled)
{
	bKeyframeEnabled = bInKeyframeEnabled;
}

bool FCustomDetailsViewItemBase::IsWidgetVisible() const
{
	for (const TSharedPtr<ICustomDetailsViewItem>& Child : GetChildren())
	{
		if (Child && Child->IsWidgetVisible())
		{
			return true;
		}
	}

	if (DetailWidgetRow.VisibilityAttr.Get(/* Default Value */ EVisibility::Visible) != EVisibility::Visible)
	{
		return false;
	}

	if (DetailWidgetRow.HasColumns())
	{
		return (DetailWidgetRow.HasNameContent() && DetailWidgetRow.NameWidget.Widget->GetVisibility().IsVisible())
			|| (DetailWidgetRow.HasValueContent() && DetailWidgetRow.ValueWidget.Widget->GetVisibility().IsVisible());
	}

	return DetailWidgetRow.HasAnyContent() && DetailWidgetRow.WholeRowWidget.Widget->GetVisibility().IsVisible();
}

void FCustomDetailsViewItemBase::SetValueWidgetWidthOverride(TOptional<float> InWidth)
{
	ValueWidthOverride = InWidth;
}

void FCustomDetailsViewItemBase::SetEnabledOverride(TAttribute<bool> InOverride)
{
	EnabledOverride = InOverride;
}

const FDetailWidgetRow& FCustomDetailsViewItemBase::GetDetailWidgetRow() const
{
	return DetailWidgetRow;
}

bool FCustomDetailsViewItemBase::CreateResetToDefaultButton(FPropertyRowExtensionButton& OutButton)
{
	return false;
}

void FCustomDetailsViewItemBase::CreateGlobalExtensionButtons(TArray<FPropertyRowExtensionButton>& OutExtensionButtons)
{
}

TSharedRef<SWidget> FCustomDetailsViewItemBase::CreateExtensionButtonWidget(const TArray<FPropertyRowExtensionButton>& InExtensionButtons)
{
	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), "DetailsView.ExtensionToolBar");
	ToolbarBuilder.SetIsFocusable(false);

	for (const FPropertyRowExtensionButton& Extension : InExtensionButtons)
	{
		ToolbarBuilder.AddToolBarButton(Extension.UIAction, NAME_None, Extension.Label, Extension.ToolTip, Extension.Icon);
	}

	TSharedRef<SWidget> ExtensionWidget = SNew(SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			ToolbarBuilder.MakeWidget()
		];

	return ExtensionWidget;
}

void FCustomDetailsViewItemBase::AddWholeRowWidget(const TSharedRef<SSplitter>& InSplitter, const TSharedPtr<SWidget>& InPrependWidget,
	const FDetailColumnSizeData& InColumnSizeData, const FMargin& InPadding)
{
	TSharedPtr<SWidget> WholeRowWidgetInner = DetailWidgetRow.WholeRowWidget.Widget;

	if (TSharedPtr<SWidget> OverrideWidget = GetOverrideWidget(ECustomDetailsViewWidgetType::WholeRow))
	{
		WholeRowWidgetInner = OverrideWidget;
	}

	if (!WholeRowWidgetInner.IsValid())
	{
		return;
	}

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	if (InPrependWidget.IsValid())
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Fill)
			[
				InPrependWidget.ToSharedRef()
			];
	}

	HorizontalBox->AddSlot()
		.FillWidth(1.f)
		[
			WholeRowWidgetInner.ToSharedRef()
		];

	TSharedRef<SWidget> WholeRowWidget = SNew(SBox)
		.Padding(InPadding)
		.HAlign(DetailWidgetRow.WholeRowWidget.HorizontalAlignment)
		.VAlign(DetailWidgetRow.WholeRowWidget.VerticalAlignment)
		.MinDesiredWidth(UE::CustomDetailsView::Private::GetOptionalSize(DetailWidgetRow.WholeRowWidget.MinWidth))
		.MaxDesiredWidth(UE::CustomDetailsView::Private::GetOptionalSize(DetailWidgetRow.WholeRowWidget.MaxWidth))
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			HorizontalBox
		];

	Widgets.Add(ECustomDetailsViewWidgetType::WholeRow, WholeRowWidget);

	if (EnabledOverride.IsSet())
	{
		WholeRowWidget->SetEnabled(EnabledOverride);
	}

	InSplitter->AddSlot()
		.Value(InColumnSizeData.GetWholeRowColumnWidth())
		.OnSlotResized(InColumnSizeData.GetOnWholeRowColumnResized())
		[
			WholeRowWidget
		];
}

void FCustomDetailsViewItemBase::AddNameWidget(const TSharedRef<SSplitter>& InSplitter, const TSharedPtr<SWidget>& InPrependWidget, const FDetailColumnSizeData& InColumnSizeData, const FMargin& InPadding)
{
	TSharedPtr<SWidget> NameWidgetInner = DetailWidgetRow.NameWidget.Widget;

	if (TSharedPtr<SWidget> OverrideWidget = GetOverrideWidget(ECustomDetailsViewWidgetType::Name))
	{
		NameWidgetInner = OverrideWidget;
	}

	if (!NameWidgetInner.IsValid())
	{
		return;
	}

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	if (InPrependWidget.IsValid())
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Fill)
			[
				InPrependWidget.ToSharedRef()
			];
	}

	// Edit Condition
	HorizontalBox->AddSlot()
		.AutoWidth()
		.Padding(2.f, 0.f, 0.f, 0.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			MakeEditConditionWidget()
		];

	// Name Widget
	HorizontalBox->AddSlot()
		.FillWidth(1.f)
		.Padding(2.f, 0.f, 0.f, 0.f)
		.HAlign(DetailWidgetRow.NameWidget.HorizontalAlignment)
		.VAlign(DetailWidgetRow.NameWidget.VerticalAlignment)
		[
			NameWidgetInner.ToSharedRef()
		];

	TSharedRef<SWidget> NameWidget = SNew(SBox)
		.Padding(InPadding)
		.MinDesiredWidth(UE::CustomDetailsView::Private::GetOptionalSize(DetailWidgetRow.NameWidget.MinWidth))
		.MaxDesiredWidth(UE::CustomDetailsView::Private::GetOptionalSize(DetailWidgetRow.NameWidget.MaxWidth))
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			HorizontalBox
		];

	if (EnabledOverride.IsSet())
	{
		HorizontalBox->SetEnabled(EnabledOverride);
	}

	Widgets.Add(ECustomDetailsViewWidgetType::Name, NameWidget);

	InSplitter->AddSlot()
		.Value(InColumnSizeData.GetNameColumnWidth())
		.OnSlotResized(InColumnSizeData.GetOnNameColumnResized())
		[
			NameWidget
		];
}

void FCustomDetailsViewItemBase::AddValueWidget(const TSharedRef<SSplitter>& InSplitter, const FDetailColumnSizeData& InColumnSizeData, const FMargin& InPadding)
{
	TSharedPtr<SWidget> ValueWidgetInner = DetailWidgetRow.ValueWidget.Widget;

	if (TSharedPtr<SWidget> OverrideWidget = GetOverrideWidget(ECustomDetailsViewWidgetType::Value))
	{
		ValueWidgetInner = OverrideWidget;
		DetailWidgetRow.ValueWidget.HorizontalAlignment = EHorizontalAlignment::HAlign_Fill;
	}

	if (!ValueWidgetInner.IsValid())
	{
		return;
	}

	TSharedPtr<SWidget> ValueWidget;

	if (!ValueWidthOverride.IsSet())
	{
		ValueWidget = SNew(SBox)
			.Padding(InPadding)
			.HAlign(DetailWidgetRow.ValueWidget.HorizontalAlignment)
			.VAlign(DetailWidgetRow.ValueWidget.VerticalAlignment)
			.MinDesiredWidth(UE::CustomDetailsView::Private::GetOptionalSize(DetailWidgetRow.ValueWidget.MinWidth))
			.MaxDesiredWidth(UE::CustomDetailsView::Private::GetOptionalSize(DetailWidgetRow.ValueWidget.MaxWidth))
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				ValueWidgetInner.ToSharedRef()
			];
	}
	else
	{
		ValueWidget = SNew(SBox)
			.Padding(InPadding)
			.HAlign(DetailWidgetRow.ValueWidget.HorizontalAlignment)
			.VAlign(DetailWidgetRow.ValueWidget.VerticalAlignment)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SBox)
				.HAlign(HAlign_Fill)
				.MinDesiredWidth(ValueWidthOverride.GetValue())
				.MaxDesiredWidth(ValueWidthOverride.GetValue())
				[
					ValueWidgetInner.ToSharedRef()
				]
			];
	}

	if (EnabledOverride.IsSet())
	{
		ValueWidgetInner->SetEnabled(EnabledOverride);
	}

	Widgets.Add(ECustomDetailsViewWidgetType::Value, ValueWidget);

	InSplitter->AddSlot()
		.Value(InColumnSizeData.GetValueColumnWidth())
		.OnSlotResized(InColumnSizeData.GetOnValueColumnResized())
		[
			ValueWidget.ToSharedRef()
		];
}

void FCustomDetailsViewItemBase::AddExtensionWidget(const TSharedRef<SSplitter>& InSplitter, const FDetailColumnSizeData& InColumnSizeData, const FCustomDetailsViewArgs& InViewArgs)
{
	TSharedRef<SWidget> ExtensionWidgetInner = SNullWidget::NullWidget;

	if (TSharedPtr<SWidget> OverrideWidget = GetOverrideWidget(ECustomDetailsViewWidgetType::Extensions))
	{
		ExtensionWidgetInner = OverrideWidget.ToSharedRef();

		if (EnabledOverride.IsSet())
		{
			ExtensionWidgetInner->SetEnabled(EnabledOverride);
		}
	}

	Widgets.Add(ECustomDetailsViewWidgetType::Extensions, SNullWidget::NullWidget);

	InSplitter->AddSlot()
		.Value(InColumnSizeData.GetRightColumnWidth())
		.MinSize(InColumnSizeData.GetRightColumnMinWidth())
		.OnSlotResized(InColumnSizeData.GetOnRightColumnResized())
		[
			ExtensionWidgetInner
		];
}

TSharedPtr<IDetailKeyframeHandler> FCustomDetailsViewItemBase::GetKeyframeHandler() const
{
	if (const TSharedPtr<SCustomDetailsView> CustomDetailsView = CustomDetailsViewWeak.Pin())
	{
		return CustomDetailsView->GetViewArgs().KeyframeHandler;
	}

	return nullptr;
}

bool FCustomDetailsViewItemBase::CanKeyframe() const
{
	if (const TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = GetKeyframeHandler())
	{
		return KeyframeHandler->IsPropertyKeyingEnabled();
	}

	return false;
}

TSharedRef<SWidget> FCustomDetailsViewItemBase::MakeEditConditionWidget()
{
	return SNullWidget::NullWidget;
}

FReply FCustomDetailsViewItemBase::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (const TSharedPtr<SWidget> MenuContent = GenerateContextMenuWidget())
		{
			FSlateApplication::Get().PushMenu(
				InPointerEvent.GetWindow(),
				FWidgetPath(),
				MenuContent.ToSharedRef(),
				InPointerEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}
