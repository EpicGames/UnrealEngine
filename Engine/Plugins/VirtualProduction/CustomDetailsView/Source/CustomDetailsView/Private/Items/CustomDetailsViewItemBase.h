// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsViewFwd.h"
#include "DetailWidgetRow.h"
#include "Items/CustomDetailsViewItemId.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Templates/SharedPointer.h"

class FDetailColumnSizeData;
class ICustomDetailsView;
class ICustomDetailsViewItem;
class SCheckBox;
class SCustomDetailsView;
class SSplitter;
enum class ECustomDetailsViewWidgetType;
enum class EDetailNodeType;
struct FCustomDetailsViewArgs;
struct FMargin;

namespace UE::CustomDetailsView::Private
{
	TAttribute<FOptionalSize> GetOptionalSize(const TOptional<float>& InOptional);
}

class FCustomDetailsViewItemBase : public ICustomDetailsViewItem
{
public:
	explicit FCustomDetailsViewItemBase(const TSharedRef<SCustomDetailsView>& InCustomDetailsView,
		const TSharedPtr<ICustomDetailsViewItem>& InParentItem = nullptr);

	virtual ~FCustomDetailsViewItemBase() override = default;

	void UpdateIndentLevel();

	//~ Begin ICustomDetailsViewItem
	virtual void InitWidget() override;
	virtual TSharedPtr<ICustomDetailsView> GetCustomDetailsView() const override;
	virtual TSharedPtr<IDetailsView> GetDetailsView() const override { return nullptr; }
	virtual const FCustomDetailsViewItemId& GetItemId() const override final;
	virtual TSharedPtr<ICustomDetailsViewItem> GetRoot() const override;
	virtual TSharedPtr<ICustomDetailsViewItem> GetParent() const override;
	virtual void SetParent(TSharedPtr<ICustomDetailsViewItem> InParent) override;
	virtual const TArray<TSharedPtr<ICustomDetailsViewItem>>& GetChildren() const override;
	virtual void RefreshChildren(TSharedPtr<ICustomDetailsViewItem> InParentOverride = nullptr) override;
	virtual TOptional<EDetailNodeType> GetNodeType() const override;
	virtual void AddAsChild(const TSharedRef<ICustomDetailsViewItem>& InParentItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) override;
	virtual TSharedRef<SWidget> MakeWidget(const TSharedPtr<SWidget>& InPrependWidget, const TSharedPtr<SWidget>& InOwningWidget) override;
	virtual TSharedPtr<SWidget> GetWidget(ECustomDetailsViewWidgetType InWidgetType) const override;
	virtual TSharedPtr<SWidget> GetOverrideWidget(ECustomDetailsViewWidgetType InWidgetType) const override;
	virtual void SetOverrideWidget(ECustomDetailsViewWidgetType InWidgetType, TSharedPtr<SWidget> InWidget) override;
	virtual void SetKeyframeEnabled(bool bInKeyframeEnabled) override;
	virtual void SetResetToDefaultOverride(const FResetToDefaultOverride& InOverride) override {}
	virtual bool IsWidgetVisible() const override;
	virtual void SetValueWidgetWidthOverride(TOptional<float> InWidth) override;
	virtual void SetEnabledOverride(TAttribute<bool> InOverride) override;
	virtual const FDetailWidgetRow& GetDetailWidgetRow() const override;
	virtual bool CreateResetToDefaultButton(FPropertyRowExtensionButton& OutButton) override;
	virtual void CreateGlobalExtensionButtons(TArray<FPropertyRowExtensionButton>& OutExtensionButtons) override;
	virtual TSharedRef<SWidget> CreateExtensionButtonWidget(const TArray<FPropertyRowExtensionButton>& InExtensionButtons) override;
	virtual void SetCreateChildItemDelegate(FOnCustomDetailsViewGenerateChildItem InDelegate) override {} // No default implementation
	virtual void SetCustomizeItemMenuContext(FOnCustomDetailsViewCustomizeItemMenuContext InDelegate) override {} // No default implementation
	//~ End ICustomDetailsViewItem

	virtual void AddWholeRowWidget(const TSharedRef<SSplitter>& InSplitter
		, const TSharedPtr<SWidget>& InPrependWidget
		, const FDetailColumnSizeData& InColumnSizeData
		, const FMargin& InPadding);

	virtual void AddNameWidget(const TSharedRef<SSplitter>& InSplitter
		, const TSharedPtr<SWidget>& InPrependWidget
		, const FDetailColumnSizeData& InColumnSizeData
		, const FMargin& InPadding);

	virtual void AddValueWidget(const TSharedRef<SSplitter>& InSplitter
		, const FDetailColumnSizeData& InColumnSizeData
		, const FMargin& InPadding);

	virtual void AddExtensionWidget(const TSharedRef<SSplitter>& InSplitter
		, const FDetailColumnSizeData& InColumnSizeData
		, const FCustomDetailsViewArgs& InViewArgs);

	TSharedPtr<IDetailKeyframeHandler> GetKeyframeHandler() const;

	bool CanKeyframe() const;

	virtual TSharedRef<SWidget> MakeEditConditionWidget();

protected:
	FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);

	TArray<TSharedPtr<ICustomDetailsViewItem>> GenerateChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem);

	virtual void InitWidget_Internal() {}

	virtual void UpdateVisibility();

	//~ Begin ICustomDetailsViewItem
	/** Generate optional context menu on row right click */
	virtual TSharedPtr<SWidget> GenerateContextMenuWidget()
	{
		return nullptr;
	}

	virtual void GenerateCustomChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) {};

	virtual void GatherChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem, const UE::CustomDetailsView::FTreeExtensionType& InTreeExtensions,
		ECustomDetailsTreeInsertPosition InPosition, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) override;
	//~ End ICustomDetailsViewItem

	/** Weak pointer to the Entity View holding this Item */
	TWeakPtr<SCustomDetailsView> CustomDetailsViewWeak;

	/** Weak pointer to the Parent Item */
	TWeakPtr<ICustomDetailsViewItem> ParentWeak;

	/** The Identifier for this Item */
	FCustomDetailsViewItemId ItemId;

	/** Cached list of Children gotten since this Item was last refreshed/generated */
	TArray<TSharedPtr<ICustomDetailsViewItem>> Children;

	/** The Widgets generated for each Widget Type */
	TMap<ECustomDetailsViewWidgetType, TSharedPtr<SWidget>> Widgets;

	/** Widgets set by the user to override automatically generated widgets */
	TMap<ECustomDetailsViewWidgetType, TSharedRef<SWidget>> OverrideWidgets;

	/** The Node Type of the Detail Tree Node. Can be unset if Detail Tree Node was never valid */
	TOptional<EDetailNodeType> NodeType;

	/** The Widget Row information Retrieved from the Detail Tree Node */
	FDetailWidgetRow DetailWidgetRow;

	/** Calculated Indent Level based on Hierarchy */
	int32 IndentLevel = -1;

	/** If false, the keyframe button for this widget will never display. */
	bool bKeyframeEnabled = true;

	/** Overrides the create value widget's maximum width, if set. */
	TOptional<float> ValueWidthOverride;

	/** Set to give custom enabled override. */
	TAttribute<bool> EnabledOverride;
};
