// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/CustomDetailsViewItemId.h"
#include "PropertyHandle.h"
#include "PropertyPath.h"
#include "UObject/UnrealType.h"

namespace UE::CustomDetailsView::Private
{
	FString MakeItemName(const FCustomDetailsViewItemId& InParentId, uint32 InItemType, const FString& InItemName)
	{
		FString JoinString = "";

		switch (InParentId.GetItemType())
		{
			// Root or unknown parent!
			case FCustomDetailsViewItemId::NullItemType:
				return InItemName;

			// Parent$Child
			case FCustomDetailsViewItemId::CustomItemType:
				JoinString = TEXT("$");
				break;

			// Category|Child
			// Parent:Child
			case static_cast<uint32>(EDetailNodeType::Category):
				JoinString = (InItemType == static_cast<uint32>(EDetailNodeType::Category))
					? TEXT("|")
					: TEXT(":");
				break;

			// Parent^SubCategory
			// Parent.Child
			default:
				JoinString = (InItemType == static_cast<uint32>(EDetailNodeType::Category))
					? TEXT("^")
					: TEXT(".");
				break;
		}

		return InParentId.GetItemName() + JoinString + InItemName;
	}
}

FCustomDetailsViewItemId FCustomDetailsViewItemId::MakeCategoryId(FName InCategoryName, const FCustomDetailsViewItemId* InParentId)
{
	return FCustomDetailsViewItemId(
		static_cast<uint32>(EDetailNodeType::Category),
		InCategoryName.ToString(),
		InParentId
	);
}

FCustomDetailsViewItemId FCustomDetailsViewItemId::MakePropertyId(FProperty* Property)
{
	if (Property)
	{
		return FCustomDetailsViewItemId(
			static_cast<uint32>(EDetailNodeType::Item),
			Property->GetPathName()
		);
	}

	return FCustomDetailsViewItemId();
}

FCustomDetailsViewItemId FCustomDetailsViewItemId::MakePropertyId(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	if (InPropertyHandle.IsValid())
	{
		return FCustomDetailsViewItemId::MakePropertyId(InPropertyHandle->GetProperty());
	}
	return FCustomDetailsViewItemId();
}

FCustomDetailsViewItemId FCustomDetailsViewItemId::MakeCustomId(FName InItemName, const FCustomDetailsViewItemId* InParentId)
{
	return FCustomDetailsViewItemId(
		CustomItemType,
		InItemName.ToString(),
		InParentId
	);
}

FCustomDetailsViewItemId FCustomDetailsViewItemId::MakeFromDetailTreeNode(const TSharedRef<IDetailTreeNode>& InDetailTreeNode,
	const FCustomDetailsViewItemId* InParentId)
{
	if (InDetailTreeNode->GetNodeType() == EDetailNodeType::Category)
	{
		return FCustomDetailsViewItemId::MakeCategoryId(InDetailTreeNode->GetNodeName(), InParentId);
	}

	if (const TSharedPtr<IPropertyHandle> PropertyHandle = InDetailTreeNode->CreatePropertyHandle())
	{
		return FCustomDetailsViewItemId::MakePropertyId(PropertyHandle);
	}

	return FCustomDetailsViewItemId(
		static_cast<uint32>(InDetailTreeNode->GetNodeType()), 
		InDetailTreeNode->GetNodeName().ToString()
	);
}

bool FCustomDetailsViewItemId::IsType(EDetailNodeType InType) const
{
	return ItemType == static_cast<uint32>(InType);
}

FCustomDetailsViewItemId::FCustomDetailsViewItemId(uint32 InItemType, const FString& InItemName, const FCustomDetailsViewItemId* InParentId)
	: ItemName(InParentId ? UE::CustomDetailsView::Private::MakeItemName(*InParentId, InItemType, InItemName) : InItemName)
	, ItemType(InItemType)
{
	CalculateTypeHash();
}
