// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomDetailsViewRootItem.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "SCustomDetailsView.h"
#include "Widgets/SNullWidget.h"

#define UE_CUSTOM_DETAILS_ROOT_ITEM_NO_ENTRY() checkf(0, TEXT("%s shouldn't be called on Root Item"), StringCast<TCHAR>(__FUNCTION__).Get())

FCustomDetailsViewRootItem::FCustomDetailsViewRootItem(const TSharedRef<SCustomDetailsView>& InCustomDetailsView)
	: FCustomDetailsViewDetailTreeNodeItem(InCustomDetailsView, nullptr, nullptr)
{
}

FCustomDetailsViewRootItem::~FCustomDetailsViewRootItem()
{
	if (PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->OnRowsRefreshed().Remove(OnRowsRefreshedHandle);
		PropertyRowGenerator->OnFinishedChangingProperties().Remove(OnFinishedChangeHandle);
		OnRowsRefreshedHandle.Reset();
		OnFinishedChangeHandle.Reset();
	}
}

void FCustomDetailsViewRootItem::RefreshItemId()
{
}

void FCustomDetailsViewRootItem::RefreshChildren(TSharedPtr<ICustomDetailsViewItem> InParentOverride)
{
	Children.Reset();

	if (!PropertyRowGenerator.IsValid())
	{
		return;
	}

	// Don't need to do anything about this, it will not affect anything. Passing in a parent is an error, though.
	ensure(!InParentOverride.IsValid());

	Children = GenerateChildren(AsShared());
}

TSharedRef<SWidget> FCustomDetailsViewRootItem::MakeWidget(const TSharedPtr<SWidget>& InPrependWidget
	, const TSharedPtr<SWidget>& InOwningWidget)
{
	UE_CUSTOM_DETAILS_ROOT_ITEM_NO_ENTRY();
	return SNullWidget::NullWidget;
}

TSharedPtr<SWidget> FCustomDetailsViewRootItem::GetWidget(ECustomDetailsViewWidgetType InWidgetType) const
{
	UE_CUSTOM_DETAILS_ROOT_ITEM_NO_ENTRY();
	return nullptr;
}

void FCustomDetailsViewRootItem::SetObject(UObject* InObject)
{
	TArray<UObject*> Objects;
	if (IsValid(InObject))
	{
		Objects.Add(InObject);
	}
	SetObjects(Objects);
}

void FCustomDetailsViewRootItem::SetObjects(const TArray<UObject*>& InObjects)
{
	if (PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->SetObjects(InObjects);
	}
}

void FCustomDetailsViewRootItem::SetStruct(const TSharedPtr<FStructOnScope>& InStruct)
{
	if (PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->SetStructure(InStruct);
	}
}

bool FCustomDetailsViewRootItem::FilterItems(const TArray<FString>& InFilterStrings)
{
	if (PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->FilterNodes(InFilterStrings);
	}

	// If all rows are hidden, nothing passed filters
	return IsWidgetVisible();
}

void FCustomDetailsViewRootItem::InitWidget_Internal()
{
	TSharedPtr<SCustomDetailsView> CustomDetailsView = CustomDetailsViewWeak.Pin();

	if (!CustomDetailsView.IsValid())
	{
		return;
	}

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	PropertyRowGenerator = PropertyEditor.CreatePropertyRowGenerator(CustomDetailsView->GetViewArgs().RowGeneratorArgs);

	OnRowsRefreshedHandle = PropertyRowGenerator->OnRowsRefreshed().AddSP(CustomDetailsView.ToSharedRef()
		, &SCustomDetailsView::RebuildTree, ECustomDetailsViewBuildType::InstantBuild);

	OnFinishedChangeHandle = PropertyRowGenerator->OnFinishedChangingProperties().AddSP(CustomDetailsView.ToSharedRef()
		, &SCustomDetailsView::OnFinishedChangingProperties);
}

void FCustomDetailsViewRootItem::GenerateCustomChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren)
{
	if (!CustomDetailsViewWeak.IsValid() || !PropertyRowGenerator.IsValid())
	{
		return;
	}

	const TArray<TSharedRef<IDetailTreeNode>>& RootTreeNodes = PropertyRowGenerator->GetRootTreeNodes();

	AddChildDetailsTreeNodes(InParentItem, ECustomDetailsViewNodePropertyFlag::None, RootTreeNodes, OutChildren);
}

#undef UE_CUSTOM_DETAILS_ROOT_ITEM_NO_ENTRY
