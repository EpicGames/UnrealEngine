// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsViewDetailTreeNodeItem.h"
#include "ICustomDetailsView.h"

class IPropertyRowGenerator;

class FCustomDetailsViewRootItem : public FCustomDetailsViewDetailTreeNodeItem, public ICustomDetailsViewBase
{
public:
	explicit FCustomDetailsViewRootItem(const TSharedRef<SCustomDetailsView>& InCustomDetailsView);
	
	virtual ~FCustomDetailsViewRootItem() override;

	//~ Begin ICustomDetailsViewItem
	virtual void RefreshItemId() override;
	virtual void RefreshChildren(TSharedPtr<ICustomDetailsViewItem> InParentOverride = nullptr) override;
	virtual TSharedRef<SWidget> MakeWidget(const TSharedPtr<SWidget>& InPrependWidget, const TSharedPtr<SWidget>& InOwningWidget) override;
	virtual TSharedPtr<SWidget> GetWidget(ECustomDetailsViewWidgetType InWidgetType) const override;
	//~ End ICustomDetailsViewItem

	//~ Begin ICustomDetailsViewBase
	virtual void SetObject(UObject* InObject) override;
	virtual void SetObjects(const TArray<UObject*>& InObjects) override;
	virtual void SetStruct(const TSharedPtr<FStructOnScope>& InStruct) override;
	virtual bool FilterItems(const TArray<FString>& InFilterStrings) override;
	//~ End ICustomDetailsViewBase

private:
	//~ Begin ICustomDetailsViewBase
	virtual void InitWidget_Internal() override;
	virtual void GenerateCustomChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) override;
	//~ End ICustomDetailsViewBase

	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	FDelegateHandle OnRowsRefreshedHandle;

	FDelegateHandle OnFinishedChangeHandle;
};
