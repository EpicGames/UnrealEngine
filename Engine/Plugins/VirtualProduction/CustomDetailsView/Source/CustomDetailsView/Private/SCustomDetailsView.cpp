// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCustomDetailsView.h"
#include "Brushes/SlateColorBrush.h"
#include "DetailColumnSizeData.h"
#include "IDetailTreeNode.h"
#include "Items/CustomDetailsViewCustomCategoryItem.h"
#include "Items/CustomDetailsViewCustomItem.h"
#include "Items/CustomDetailsViewDetailTreeNodeItem.h"
#include "Items/CustomDetailsViewRootItem.h"
#include "Items/ICustomDetailsViewCustomCategoryItem.h"
#include "Slate/SCustomDetailsTreeView.h"
#include "Slate/SCustomDetailsViewItemRow.h"
#include "Styling/StyleColors.h"
#include "Widgets/SInvalidationPanel.h"

void SCustomDetailsView::Construct(const FArguments& InArgs, const FCustomDetailsViewArgs& InCustomDetailsViewArgs)
{
	SetCanTick(false);

	ViewArgs = InCustomDetailsViewArgs;

	if (!ViewArgs.ColumnSizeData.IsValid())
	{
		ViewArgs.ColumnSizeData = MakeShared<FDetailColumnSizeData>();

		FDetailColumnSizeData& ColumnSizeData = *ViewArgs.ColumnSizeData;
		ColumnSizeData.SetValueColumnWidth(ViewArgs.ValueColumnWidth);
		ColumnSizeData.SetRightColumnMinWidth(ViewArgs.RightColumnMinWidth);
	}

	const TSharedRef<SCustomDetailsView> This = SharedThis(this);

	RootItem = MakeShared<FCustomDetailsViewRootItem>(This);
	RootItem->InitWidget();

	ChildSlot
	[
		SAssignNew(ViewTree, SCustomDetailsTreeView)
		.TreeItemsSource(&RootItem->GetChildren())
		.OnGetChildren(this, &SCustomDetailsView::OnGetChildren)
		.OnExpansionChanged(this, &SCustomDetailsView::OnExpansionChanged)
		.OnSetExpansionRecursive(this, &SCustomDetailsView::SetExpansionRecursive)
		.OnGenerateRow(this, &SCustomDetailsView::OnGenerateRow)
		.SelectionMode(ESelectionMode::None)
		.ExternalScrollbar(ViewArgs.ExternalScrollBar)
	];

	const FLinearColor PanelColor = FSlateColor(EStyleColor::Panel).GetSpecifiedColor();
	BackgroundBrush = MakeShared<FSlateColorBrush>(FLinearColor(PanelColor.R, PanelColor.G, PanelColor.B, ViewArgs.TableBackgroundOpacity));
	ViewTree->SetBackgroundBrush(BackgroundBrush.Get());
	ViewTree->SetCustomDetailsView(This);
}

void SCustomDetailsView::Refresh()
{
	if (!ViewTree.IsValid())
    {
		return;
    }

	TArray<TSharedPtr<ICustomDetailsViewItem>> ItemsRemaining = RootItem->GetChildren();

	// Update Item Expansions
	while (!ItemsRemaining.IsEmpty())
	{
		const TSharedPtr<ICustomDetailsViewItem> Item = ItemsRemaining.Pop();
		if (!Item.IsValid())
		{
			continue;
		}

		ViewTree->SetItemExpansion(Item, ShouldItemExpand(Item));
		ItemsRemaining.Append(Item->GetChildren());
	}

	ViewTree->RequestTreeRefresh();
}

void SCustomDetailsView::OnTreeViewRegenerated()
{
	ViewArgs.OnTreeViewRegenerated.Broadcast();
}

void SCustomDetailsView::OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	ViewArgs.OnFinishedChangingProperties.Broadcast(InPropertyChangedEvent);
}

UE::CustomDetailsView::Private::EAllowType SCustomDetailsView::GetAllowType(const TSharedRef<ICustomDetailsViewItem>& InParentItem, 
	const TSharedRef<IDetailTreeNode>& InDetailTreeNode, ECustomDetailsViewNodePropertyFlag InNodePropertyFlags) const
{
	using namespace UE::CustomDetailsView::Private;

	const bool bIgnoreFilters = ViewArgs.bExcludeStructChildPropertiesFromFilters &&
		EnumHasAnyFlags(InNodePropertyFlags, ECustomDetailsViewNodePropertyFlag::HasParentStruct);

	const EDetailNodeType NodeType = InDetailTreeNode->GetNodeType();
	const FName NodeName = InDetailTreeNode->GetNodeName();

	switch (NodeType)
	{
		case EDetailNodeType::Advanced:
			return EAllowType::DisallowSelf;

		case EDetailNodeType::Category:
			// Check Category Allow List first since it has the most severe Result
			if (!bIgnoreFilters && !ViewArgs.CategoryAllowList.IsAllowed(NodeName))
			{
				return EAllowType::DisallowSelfAndChildren;
			}
			if (!ViewArgs.bShowCategories)
			{
				return EAllowType::DisallowSelf;
			}
			break;

		default:
			break;
	}

	const FCustomDetailsViewItemId ItemId = FCustomDetailsViewItemId::MakeFromDetailTreeNode(InDetailTreeNode, &InParentItem->GetItemId());

	if (!bIgnoreFilters && !ViewArgs.ItemAllowList.IsAllowed(ItemId))
	{
		return EAllowType::DisallowSelfAndChildren;
	}

	return EAllowType::Allowed;
}

void SCustomDetailsView::OnGetChildren(TSharedPtr<ICustomDetailsViewItem> InItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) const
{
	if (InItem.IsValid())
	{
		OutChildren.Append(InItem->GetChildren());
	}
}

void SCustomDetailsView::OnExpansionChanged(TSharedPtr<ICustomDetailsViewItem> InItem, bool bInExpanded)
{
	if (!InItem.IsValid())
	{
		return;
	}

	if (const ECustomDetailsViewExpansion* Expansion = ViewArgs.ExpansionState.Find(InItem->GetItemId()))
	{
		const bool bExpanded = *Expansion != ECustomDetailsViewExpansion::Collapsed;

		if (bExpanded)
		{
			ViewArgs.ExpansionState.Add(InItem->GetItemId(), bInExpanded ? *Expansion : ECustomDetailsViewExpansion::Collapsed);
		}
		else
		{
			ViewArgs.ExpansionState.Add(InItem->GetItemId(), bInExpanded ? ECustomDetailsViewExpansion::SelfExpanded : ECustomDetailsViewExpansion::Collapsed);
		}
	}
	else
	{
		ViewArgs.ExpansionState.Add(InItem->GetItemId(), bInExpanded ? ECustomDetailsViewExpansion::SelfExpanded : ECustomDetailsViewExpansion::Collapsed);
	}

	ViewArgs.OnExpansionStateChanged.Broadcast(InItem.ToSharedRef(), bInExpanded);
}

void SCustomDetailsView::SetExpansionRecursive(TSharedPtr<ICustomDetailsViewItem> InItem, bool bInExpand)
{
	if (!InItem.IsValid() || !ViewTree.IsValid())
	{
		return;
	}

	ViewTree->SetItemExpansion(InItem, bInExpand);
	for (const TSharedPtr<ICustomDetailsViewItem>& ChildItem : InItem->GetChildren())
	{
		if (ChildItem.IsValid())
		{
			SetExpansionRecursive(ChildItem, bInExpand);
		}
	}
}

bool SCustomDetailsView::ShouldItemExpand(const TSharedPtr<ICustomDetailsViewItem>& InItem) const
{
	if (!InItem.IsValid())
	{
		return false;
	}

	TSharedPtr<ICustomDetailsViewItem> CheckItem = InItem;

	do
	{
		if (const ECustomDetailsViewExpansion* const FoundExpansionState = ViewArgs.ExpansionState.Find(CheckItem->GetItemId()))
		{
			if (CheckItem.Get() == InItem.Get())
			{
				if (*FoundExpansionState == ECustomDetailsViewExpansion::Collapsed)
				{
					return false;
				}

				return true;
			}

			if (*FoundExpansionState == ECustomDetailsViewExpansion::SelfAndChildrenExpanded)
			{
				return true;
			}

			return false;
		}

		CheckItem = CheckItem->GetParent();
	}
	while (CheckItem.IsValid() && CheckItem->GetParent());

	return ViewArgs.bDefaultItemsExpanded;
}

TSharedRef<ITableRow> SCustomDetailsView::OnGenerateRow(TSharedPtr<ICustomDetailsViewItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const
{
	return SNew(SCustomDetailsViewItemRow, InOwnerTable, InItem, ViewArgs);
}

void SCustomDetailsView::SetObject(UObject* InObject)
{
	RootItem->SetObject(InObject);
}

void SCustomDetailsView::SetObjects(const TArray<UObject*>& InObjects)
{
	RootItem->SetObjects(InObjects);
}

void SCustomDetailsView::SetStruct(const TSharedPtr<FStructOnScope>& InStruct)
{
	RootItem->SetStruct(InStruct);
}

TSharedRef<ICustomDetailsViewItem> SCustomDetailsView::GetRootItem() const
{
	return RootItem.ToSharedRef();
}

TSharedPtr<ICustomDetailsViewItem> SCustomDetailsView::FindItem(const FCustomDetailsViewItemId& InItemId) const
{
	if (const TSharedRef<ICustomDetailsViewItem>* const FoundItem = ItemMap.Find(InItemId))
	{
		return *FoundItem;
	}
	return nullptr;
}

TSharedRef<STreeView<TSharedPtr<ICustomDetailsViewItem>>> SCustomDetailsView::MakeSubTree(const TArray<TSharedPtr<ICustomDetailsViewItem>>* InSourceItems) const
{
	return SNew(STreeView<TSharedPtr<ICustomDetailsViewItem>>)
		.TreeItemsSource(InSourceItems)
		.OnGetChildren(this, &SCustomDetailsView::OnGetChildren)
		.OnGenerateRow(this, &SCustomDetailsView::OnGenerateRow)
		.SelectionMode(ESelectionMode::None);
}

void SCustomDetailsView::RebuildTree(ECustomDetailsViewBuildType InBuildType)
{
	if (ShouldRebuildImmediately(InBuildType))
	{
		bPendingRebuild = false;
		ItemMap.Reset();
		RootItem->RefreshChildren();
		Refresh();
	}
	else if (bPendingRebuild == false)
	{
		bPendingRebuild = true;
		TWeakPtr<SCustomDetailsView> CustomDetailsViewWeak = SharedThis(this);
		RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateLambda([CustomDetailsViewWeak](double, float)->EActiveTimerReturnType
			{
				TSharedPtr<SCustomDetailsView> CustomDetailsView = CustomDetailsViewWeak.Pin();
				if (CustomDetailsView.IsValid() && CustomDetailsView->bPendingRebuild)
				{
					CustomDetailsView->RebuildTree(ECustomDetailsViewBuildType::InstantBuild);
				}
				return EActiveTimerReturnType::Stop;
			}));
	}
}

void SCustomDetailsView::ExtendTree(FCustomDetailsViewItemId InHook, ECustomDetailsTreeInsertPosition InPosition, TSharedRef<ICustomDetailsViewItem> InItem)
{
	ExtensionMap.FindOrAdd(InHook).FindOrAdd(InPosition).Emplace(InItem);
}

const UE::CustomDetailsView::FTreeExtensionType& SCustomDetailsView::GetTreeExtensions(FCustomDetailsViewItemId InHook) const
{
	if (const UE::CustomDetailsView::FTreeExtensionType* ItemExtensionMap = ExtensionMap.Find(InHook))
	{
		return *ItemExtensionMap;
	}

	static UE::CustomDetailsView::FTreeExtensionType EmptyMap;
	return EmptyMap;
}

TSharedRef<ICustomDetailsViewItem> SCustomDetailsView::CreateDetailTreeItem(TSharedRef<ICustomDetailsViewItem> InParent, 
	TSharedRef<IDetailTreeNode> InDetailTreeNode)
{
	TSharedRef<ICustomDetailsViewItem> NewItem = MakeShared<FCustomDetailsViewDetailTreeNodeItem>(SharedThis(this), InParent, InDetailTreeNode);
	NewItem->InitWidget();

	return NewItem;
}

TSharedPtr<ICustomDetailsViewCustomItem> SCustomDetailsView::CreateCustomItem(TSharedRef<ICustomDetailsViewItem> InParent, FName InItemName, const FText& InLabel, const FText& InToolTip)
{
	if (AddedCustomItems.Contains(InItemName))
	{
		return nullptr;
	}

	TSharedRef<ICustomDetailsViewCustomItem> NewCustomItem = MakeShared<FCustomDetailsViewCustomItem>(SharedThis(this), InParent, InItemName, InLabel, InToolTip);
	TSharedRef<ICustomDetailsViewItem> NewItem = NewCustomItem->AsItem();
	NewItem->InitWidget();
	AddedCustomItems.Add(InItemName, NewItem);

	return NewCustomItem;
}

TSharedPtr<ICustomDetailsViewCustomCategoryItem> SCustomDetailsView::CreateCustomCategoryItem(TSharedRef<ICustomDetailsViewItem> InParent, FName InItemName, const FText& InLabel, const FText& InToolTip)
{
	if (AddedCustomItems.Contains(InItemName))
	{
		return nullptr;
	}

	TSharedRef<ICustomDetailsViewCustomCategoryItem> NewCustomCategoryItem = MakeShared<FCustomDetailsViewCustomCategoryItem>(SharedThis(this), InParent, InItemName, InLabel, InToolTip);
	TSharedRef<ICustomDetailsViewItem> NewItem = NewCustomCategoryItem->AsItem();
	NewItem->InitWidget();

	// Auto expand
	OnExpansionChanged(NewItem, true);

	if (ViewTree.IsValid())
	{
		ViewTree->SetItemExpansion(NewItem, true);
	}

	AddedCustomItems.Add(InItemName, NewItem);

	return NewCustomCategoryItem;
}

TSharedPtr<ICustomDetailsViewItem> SCustomDetailsView::FindCustomItem(const FName& InItemName) const
{
	if (const TSharedRef<ICustomDetailsViewItem>* const FoundItem = AddedCustomItems.Find(InItemName))
	{
		return *FoundItem;
	}
	return nullptr;
}

bool SCustomDetailsView::FilterItems(const TArray<FString>& InFilterStrings)
{
	if (RootItem.IsValid())
	{
		return RootItem->FilterItems(InFilterStrings);
	}

	return false;
}

bool SCustomDetailsView::GetItemExpansionState(const FCustomDetailsViewItemId& InItemId, ECustomDetailsViewExpansion& OutExpansion) const
{
	if (const ECustomDetailsViewExpansion* State = ViewArgs.ExpansionState.Find(InItemId))
	{
		OutExpansion = *State;
		return true;
	}

	return false;
}

void SCustomDetailsView::SetItemExpansionState(const FCustomDetailsViewItemId& InItemId, ECustomDetailsViewExpansion InExpansion)
{
	ViewArgs.ExpansionState.FindOrAdd(InItemId) = InExpansion;
}

bool SCustomDetailsView::ShouldRebuildImmediately(ECustomDetailsViewBuildType InBuildType) const
{
	switch (InBuildType)
	{
	// For Auto, it will only build Immediate if we need to fill / re-fill the Item Map
	case ECustomDetailsViewBuildType::Auto:
		return ItemMap.IsEmpty();

	case ECustomDetailsViewBuildType::InstantBuild:
		return true;
	}

	check(InBuildType == ECustomDetailsViewBuildType::DeferredBuild);
	return false;
}
