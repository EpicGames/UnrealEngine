// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewFwd.h"
#include "Containers/ContainersFwd.h"
#include "ICustomDetailsView.h"
#include "Items/CustomDetailsViewCustomItem.h"
#include "Items/CustomDetailsViewDetailTreeNodeItem.h"
#include "Templates/SharedPointer.h"

class SCustomDetailsTreeView;
class FCustomDetailsViewRootItem;
class IDetailTreeNode;
class ITableRow;
class STableViewBase;

namespace UE::CustomDetailsView::Private
{
	enum class EAllowType : uint8
	{
		Allowed,
		DisallowSelf,
		DisallowSelfAndChildren,
	};
}

class SCustomDetailsView : public ICustomDetailsView
{
public:
	SLATE_BEGIN_ARGS(SCustomDetailsView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FCustomDetailsViewArgs& InCustomDetailsViewArgs);

	template<typename InItemType, typename ...InArgTypes
		, typename = typename TEnableIf<TIsDerivedFrom<InItemType, ICustomDetailsViewItem>::Value>::Type>
	TSharedRef<InItemType> CreateItem(InArgTypes&&... InArgs)
	{
		TSharedRef<InItemType> Item = MakeShared<InItemType>(Forward<InArgTypes>(InArgs)...);
		Item->InitWidget();
		ItemMap.Add(Item->GetItemId(), Item);
		return Item;
	}

	void Refresh();
	
	void OnTreeViewRegenerated();

	void OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent);

	UE::CustomDetailsView::Private::EAllowType GetAllowType(const TSharedRef<ICustomDetailsViewItem>& InParentItem, 
		const TSharedRef<IDetailTreeNode>& InDetailTreeNode, ECustomDetailsViewNodePropertyFlag InNodePropertyFlags) const;

	void OnGetChildren(TSharedPtr<ICustomDetailsViewItem> InItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) const;

	void OnExpansionChanged(TSharedPtr<ICustomDetailsViewItem> InItem, bool bInExpanded);

	DLLEXPORT void SetExpansionRecursive(TSharedPtr<ICustomDetailsViewItem> InItem, bool bInExpand);

	bool ShouldItemExpand(const TSharedPtr<ICustomDetailsViewItem>& InItem) const;

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<ICustomDetailsViewItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const;

	const FCustomDetailsViewArgs& GetViewArgs() const { return ViewArgs; }

	//~ Begin ICustomDetailsViewBase
	virtual void SetObject(UObject* InObject) override;
	virtual void SetObjects(const TArray<UObject*>& InObjects) override;
	virtual void SetStruct(const TSharedPtr<FStructOnScope>& InStruct) override;
	//~ End ICustomDetailsViewBase

	//~ Begin ICustomDetailsView
	virtual TSharedRef<ICustomDetailsViewItem> GetRootItem() const override;
	virtual TSharedPtr<ICustomDetailsViewItem> FindItem(const FCustomDetailsViewItemId& InItemId) const override;
	virtual TSharedRef<STreeView<TSharedPtr<ICustomDetailsViewItem>>> MakeSubTree(const TArray<TSharedPtr<ICustomDetailsViewItem>>* InSourceItems) const override;
	virtual void RebuildTree(ECustomDetailsViewBuildType InBuildType) override;
	virtual void ExtendTree(FCustomDetailsViewItemId InHook, ECustomDetailsTreeInsertPosition InPosition, TSharedRef<ICustomDetailsViewItem> InItem) override;
	virtual const UE::CustomDetailsView::FTreeExtensionType& GetTreeExtensions(FCustomDetailsViewItemId InHook) const override;
	virtual TSharedRef<ICustomDetailsViewItem> CreateDetailTreeItem(TSharedRef<ICustomDetailsViewItem> InParent, TSharedRef<IDetailTreeNode> InDetailTreeNode) override;
	virtual TSharedPtr<ICustomDetailsViewCustomItem> CreateCustomItem(TSharedRef<ICustomDetailsViewItem> InParent, FName InItemName, const FText& InLabel = FText::GetEmpty(), const FText& InToolTip = FText::GetEmpty()) override;
	virtual TSharedPtr<ICustomDetailsViewCustomCategoryItem> CreateCustomCategoryItem(TSharedRef<ICustomDetailsViewItem> InParent, FName InItemName, const FText& InLabel = FText::GetEmpty(), const FText& InToolTip = FText::GetEmpty()) override;
	virtual TSharedPtr<ICustomDetailsViewItem> FindCustomItem(const FName& InItemName) const override;
	virtual bool FilterItems(const TArray<FString>& InFilterStrings) override;
	virtual bool GetItemExpansionState(const FCustomDetailsViewItemId& InItemId, ECustomDetailsViewExpansion& OutExpansion) const override;
	virtual void SetItemExpansionState(const FCustomDetailsViewItemId& InItemId, ECustomDetailsViewExpansion InExpansion) override;
	//~ End ICustomDetailsView

private:
	bool ShouldRebuildImmediately(ECustomDetailsViewBuildType InBuildType) const;
	
	/** Single Root of Tree, not part of the visual part Tree Widget, but provides things like the Root Items (i.e. its Children) */
	TSharedPtr<FCustomDetailsViewRootItem> RootItem;

	TMap<FName, TSharedRef<ICustomDetailsViewItem>> AddedCustomItems;

	TMap<FCustomDetailsViewItemId, TSharedRef<ICustomDetailsViewItem>> ItemMap;	

	TSharedPtr<SCustomDetailsTreeView> ViewTree;

	FCustomDetailsViewArgs ViewArgs;

	bool bPendingRebuild = true;

	TMap<FCustomDetailsViewItemId, UE::CustomDetailsView::FTreeExtensionType> ExtensionMap;

	TSharedPtr<FSlateBrush> BackgroundBrush;
};
