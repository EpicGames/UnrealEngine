// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCustomDetailsTreeView.h"
#include "SCustomDetailsView.h"

namespace UE::CustomDetailsView::Private
{
	class FRegenerateItemsScope
	{
	public:
		FRegenerateItemsScope(TWeakPtr<SCustomDetailsView> InCustomDetailsViewWeak)
			: CustomDetailsViewWeak(InCustomDetailsViewWeak)
		{
		}
	
		~FRegenerateItemsScope()
		{
			if (TSharedPtr<SCustomDetailsView> CustomDetailsView = CustomDetailsViewWeak.Pin())
			{
				CustomDetailsView->OnTreeViewRegenerated();
			}
		}
	private:
		TWeakPtr<SCustomDetailsView> CustomDetailsViewWeak;
	};
}

void SCustomDetailsTreeView::SetCustomDetailsView(const TSharedRef<SCustomDetailsView>& InCustomDetailsView)
{
	CustomDetailsViewWeak = InCustomDetailsView;
}

STableViewBase::FReGenerateResults SCustomDetailsTreeView::ReGenerateItems(const FGeometry& MyGeometry)
{
	UE::CustomDetailsView::Private::FRegenerateItemsScope RegenerateItemsScope(CustomDetailsViewWeak);
	return STreeView<TSharedPtr<ICustomDetailsViewItem>>::ReGenerateItems(MyGeometry);
}
