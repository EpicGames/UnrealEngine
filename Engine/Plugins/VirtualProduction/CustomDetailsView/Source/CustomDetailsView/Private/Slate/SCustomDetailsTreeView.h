// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsViewFwd.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/STreeView.h"

class SCustomDetailsView;

class SCustomDetailsTreeView : public STreeView<TSharedPtr<ICustomDetailsViewItem>>
{
public:
	void SetCustomDetailsView(const TSharedRef<SCustomDetailsView>& InCustomDetailsView);
	
	//~ Begin STableViewBase
	virtual FReGenerateResults ReGenerateItems(const FGeometry& MyGeometry) override;
	//~ End STableViewBase

private:
	TWeakPtr<SCustomDetailsView> CustomDetailsViewWeak;
};
