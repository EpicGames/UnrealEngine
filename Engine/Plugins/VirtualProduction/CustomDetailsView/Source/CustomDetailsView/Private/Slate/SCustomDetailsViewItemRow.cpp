// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCustomDetailsViewItemRow.h"

#include "DetailWidgetRow.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/STreeView.h"

void SCustomDetailsViewItemRow::Construct(const FArguments& InArgs
	, const TSharedRef<STableViewBase>& InOwnerTable
	, const TSharedPtr<ICustomDetailsViewItem>& InItem
	, const FCustomDetailsViewArgs& InViewArgs)
{
	check(InItem.IsValid());

	TSharedRef<SExpanderArrow> ExpanderArrow = SNew(SExpanderArrow, SharedThis(this))
		.StyleSet(InArgs._TableRowArgs._ExpanderStyleSet)
		.ShouldDrawWires(false)
		.IndentAmount(InViewArgs.IndentAmount);

	ExpanderArrowWidget = ExpanderArrow;

	TSharedRef<SWidget> Child = InItem->MakeWidget(ExpanderArrow, SharedThis(this));

	if (InItem->GetDetailWidgetRow().VisibilityAttr.IsSet() && Child != SNullWidget::NullWidget)
	{
		Child->SetVisibility(InItem->GetDetailWidgetRow().VisibilityAttr);
	}

	STableRow<TSharedPtr<ICustomDetailsViewItem>>::Construct(STableRow<TSharedPtr<ICustomDetailsViewItem>>::FArguments(InArgs._TableRowArgs)
		[
			Child
		]
		, InOwnerTable);

	SetBorderImage(FAppStyle::GetBrush("WhiteBrush"));
	SetBorderBackgroundColor(FLinearColor(1.f, 1.f, 1.f, InViewArgs.RowBackgroundOpacity));
}

void SCustomDetailsViewItemRow::ConstructChildren(ETableViewMode::Type InOwnerTableMode
	, const TAttribute<FMargin>& InPadding
	, const TSharedRef<SWidget>& InContent)
{
	check(InOwnerTableMode == ETableViewMode::Tree);
	InnerContentSlot = nullptr;
	SetContent(InContent);
}
