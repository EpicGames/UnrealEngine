// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsViewFwd.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

struct FCustomDetailsViewArgs;

class SCustomDetailsViewItemRow : public STableRow<TSharedPtr<ICustomDetailsViewItem>>
{
public:
	SLATE_BEGIN_ARGS(SCustomDetailsViewItemRow) {}
		SLATE_ARGUMENT(STableRow<TSharedPtr<ICustomDetailsViewItem>>::FArguments, TableRowArgs)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<STableViewBase>& InOwnerTable
		, const TSharedPtr<ICustomDetailsViewItem>& InItem
		, const FCustomDetailsViewArgs& InViewArgs);

	//~ Begin STableRow
	virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode
		, const TAttribute<FMargin>& InPadding
		, const TSharedRef<SWidget>& InContent) override;
	//~ End STableRow
};
