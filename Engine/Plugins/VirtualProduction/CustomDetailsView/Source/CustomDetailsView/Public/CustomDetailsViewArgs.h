// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/EqualityComparable.h"
#include "Concepts/GetTypeHashable.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CustomDetailsViewFwd.h"
#include "Delegates/Delegate.h"
#include "IPropertyRowGenerator.h"
#include "Items/CustomDetailsViewItemId.h"
#include "Layout/Margin.h"
#include "Misc/Optional.h"
#include "PropertyEditorDelegates.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FDetailColumnSizeData;
class IDetailKeyframeHandler;
class SScrollBar;

namespace UE::CustomDetailsView
{
	/**
	 * Describes a type that can be used as a Key in a Container (Set/Map). Requires a GetTypeHash overload and be Equality Comparable
	 * NOTE: Equality Comparable is required so for A and B both of type T, not only does A==B need to be implemented, but also A!=B,
	 * even though Containers like Sets/Maps only require operator==
	 */
	struct CContainerKeyable
	{
		template <typename T>
		auto Requires() -> decltype(
			Refines<CEqualityComparable, T>(),
			Refines<CGetTypeHashable, T>()
		);
	};

	/**
	 * Has similar approach to FSceneView's Show / Hidden Primitives list, in that there's:
	 * Allowed Items: Anything not in this list is not allowed. This only applies if this is set (set via calling Allow(...))
	 * Disallowed Items: Anything in this list will not be allowed, even if it is on the Allowed Item List
	 */
	template<typename InItemType, typename = typename TEnableIf<TModels<CContainerKeyable, InItemType>::Value>::Type>
	class TAllowList
	{
	public:
		void Allow(const InItemType& InItem)
		{
			if (!AllowedItems.IsSet())
			{
				AllowedItems = TSet<InItemType>();
			}
			AllowedItems->Add(InItem);
		}

		void ResetAllowedItems()
		{
			AllowedItems.Reset();
		}

		void Disallow(const InItemType& InItem)
		{
			DisallowedItems.Add(InItem);
		}

		void ResetDisallowedList()
		{
			DisallowedItems.Reset();
		}

		bool IsAllowed(const InItemType& InItem) const
		{
			return !((AllowedItems.IsSet() && !AllowedItems->Contains(InItem)) || DisallowedItems.Contains(InItem));
		}

	private:
		TOptional<TSet<InItemType>> AllowedItems;
		TSet<InItemType> DisallowedItems;
	};
}

/** The types of Widget this Custom Details View deals with */
enum class ECustomDetailsViewWidgetType
{
	Name,
	Value,
	WholeRow,
	Extensions,
};

/** Expansion state allowed for an item */
enum class ECustomDetailsViewExpansion : uint8
{
	Collapsed,
	SelfExpanded,
	SelfAndChildrenExpanded
};

struct FCustomDetailsViewArgs
{
	/** List of Allowed and Disallowed Categories */
	UE::CustomDetailsView::TAllowList<FName> CategoryAllowList;

	/** List of Allowed and Disallowed Items based on their Id */
	UE::CustomDetailsView::TAllowList<FCustomDetailsViewItemId> ItemAllowList;

	/** List of Allowed Widget Types */
	UE::CustomDetailsView::TAllowList<ECustomDetailsViewWidgetType> WidgetTypeAllowList;

	/** The default expansion state if not overriden by the Expansion State map below */
	bool bDefaultItemsExpanded = false;

	/** Map of the Node Name to their Expanded State */
	TMap<FCustomDetailsViewItemId, ECustomDetailsViewExpansion> ExpansionState;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnExpansionStateChanged, const TSharedRef<ICustomDetailsViewItem>&, bool)
	FOnExpansionStateChanged OnExpansionStateChanged;

	/** Default value column width, as a percentage, 0-1. */
	float ValueColumnWidth = 0.6f;

	/** Minimum width of the right column in Slate units. */
	float RightColumnMinWidth = 40.f;

	/**
	 * Optional: Set a Column Size Data to use. If null, Custom Details will instantiate one separately.
	 * This is recommended when dealing with multiple Custom Details Views that want to synchronize Column Sizes
	 */
	TSharedPtr<FDetailColumnSizeData> ColumnSizeData;

	/** Optional external Scroll bar to use for the Tree */
	TSharedPtr<SScrollBar> ExternalScrollBar;

	/** Arguments used for the Property Row Generator */
	FPropertyRowGeneratorArgs RowGeneratorArgs;

	/** The Keyframe Handler to use. bAllowGlobalExtensions must be set to true */
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler;

	/** Delegate called everytime a Widget has been generated for a given Item */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnItemWidgetGenerated, TSharedPtr<ICustomDetailsViewItem>);
	FOnItemWidgetGenerated OnItemWidgetGenerated;

	/** Delegate called when all the widgets in the Main Tree View has been regenerated */
	FSimpleMulticastDelegate OnTreeViewRegenerated;

	/** Delegate called when Properties have finished changing */
	FOnFinishedChangingProperties OnFinishedChangingProperties;

	/** The Padding to use for each Widget (Name, Value, etc) */
	FMargin DefaultPadding = FMargin(5.f, 2.f);

	/** Amount of indentation to make per level in the Details View Tree */
	float IndentAmount = 10.f;

	/** Opacity for the Table Background Panel */
	float TableBackgroundOpacity = 1.f;

	/** Opacity for the Row Background Panel */
	float RowBackgroundOpacity = 1.f;

	/** Whether to show Categories */
	bool bShowCategories = true;

	/** Whether to allow Reset to Default to be added in */
	bool bAllowResetToDefault = true;

	/** Whether to allow Global Extensions to be added */
	bool bAllowGlobalExtensions = false;

	/** Whether to exclude struct child properties, and their children, from filters. */
	bool bExcludeStructChildPropertiesFromFilters = true;
};

enum class ECustomDetailsViewNodePropertyFlag : uint8
{
	None            = 0,

	/** A parent node of this property is a struct property. */
	HasParentStruct = 1 << 0
};
ENUM_CLASS_FLAGS(ECustomDetailsViewNodePropertyFlag);
