// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointerFwd.h"

class FCustomDetailsViewItemId;
class ICustomDetailsViewItem;
enum class ECustomDetailsTreeInsertPosition : uint8;

namespace UE::CustomDetailsView
{
	using FTreeExtensionType = TMap<ECustomDetailsTreeInsertPosition, TArray<TSharedPtr<ICustomDetailsViewItem>>>;
}
