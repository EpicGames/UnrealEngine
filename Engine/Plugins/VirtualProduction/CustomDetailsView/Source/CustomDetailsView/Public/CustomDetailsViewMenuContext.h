// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "CustomDetailsViewMenuContext.generated.h"

UCLASS(MinimalAPI)
class UCustomDetailsViewMenuContext : public UObject
{
	GENERATED_BODY()
};
