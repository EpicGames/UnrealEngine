// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsViewArgs.h"
#include "ICustomDetailsView.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class ICustomDetailsViewModule : public IModuleInterface
{
public:
	static ICustomDetailsViewModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ICustomDetailsViewModule>("CustomDetailsView");
	}

	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded("CustomDetailsView");
	}

	virtual TSharedRef<ICustomDetailsView> CreateCustomDetailsView(const FCustomDetailsViewArgs& InArgs) = 0;
};
