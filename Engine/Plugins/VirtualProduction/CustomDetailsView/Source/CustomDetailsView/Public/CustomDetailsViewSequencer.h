// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"

#define UE_API CUSTOMDETAILSVIEW_API

class IDetailKeyframeHandler;
class IPropertyHandle;
struct FPropertyRowExtensionButton;

struct FCustomDetailsViewSequencerUtils
{
	DECLARE_DELEGATE_RetVal(TSharedPtr<IDetailKeyframeHandler>, FGetKeyframeHandlerDelegate);

	static UE_API void CreateSequencerExtensionButton(const FGetKeyframeHandlerDelegate& InKeyframeHandlerDelegate, TSharedPtr<IPropertyHandle> InPropertyHandle,
		TArray<FPropertyRowExtensionButton>& OutExtensionButtons);

	static UE_API void CreateSequencerExtensionButton(TWeakPtr<IDetailKeyframeHandler> InKeyframeHandlerWeak, TSharedPtr<IPropertyHandle> InPropertyHandle,
		TArray<FPropertyRowExtensionButton>& OutExtensionButtons);
};

#undef UE_API
