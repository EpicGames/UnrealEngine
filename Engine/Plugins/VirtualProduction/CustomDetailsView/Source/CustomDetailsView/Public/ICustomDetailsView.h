// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsViewFwd.h"
#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FStructOnScope;
class ICustomDetailsViewCustomCategoryItem;
class ICustomDetailsViewCustomItem;
class IDetailTreeNode;
class UObject;
template<typename ItemType> class STreeView;
enum class ECustomDetailsViewExpansion : uint8;

enum class ECustomDetailsViewBuildType : uint8
{
	/** Custom Details View determines whether it should Build Instantly or Deferred */
	Auto,
	/** Force an Instant Build. Useful for when you want to see the results of the Build right after */
	InstantBuild,
	/** Defer the Build to a later point in time. Useful to avoid having to rebuild multiple times (e.g. when resizing the widget containing the tree view) */
	DeferredBuild,
};

enum class ECustomDetailsTreeInsertPosition : uint8
{
	/** Adds an item into the hierarchy on the same level as the current item but before it in the order. */
	Before,
	/** Adds an item into the hierarchy on the same level as the current item but after it in the order. */
	After,
	/** Prepends an item to the current item's child items. */
	FirstChild,
	/** Appends an item to the current item's child items. */
	Child,
	/** Appends an item to the current item's child items, but after the list above. */
	LastChild
};

class ICustomDetailsViewBase
{
public:
	virtual ~ICustomDetailsViewBase() = default;

	virtual void SetObject(UObject* InObject) = 0;

	virtual void SetObjects(const TArray<UObject*>& InObjects) = 0;

	virtual void SetStruct(const TSharedPtr<FStructOnScope>& InStruct) = 0;

	/**
	 * Applies a row filter on this tree items,
	 * returns true when filter found something
	 * returns false when no rows passed filters 
	 */
	virtual bool FilterItems(const TArray<FString>& InFilterStrings) = 0;
};

class ICustomDetailsView : public SCompoundWidget, public ICustomDetailsViewBase
{
public:
	/**
	 * Get the Root Item of the Main Tree.
	 * WARNING: This Item is not meant to make Widgets or have an Item Id. It's purely for having one source to house the entire tree
	 */
	virtual TSharedRef<ICustomDetailsViewItem> GetRootItem() const = 0;

	/**
	 * Quickly find an Item with the given Item Id without iterating the tree
	 * @param InItemId the id of the item to look for
	 * @return pointer to the item, or null if it was not found
	 */
	virtual TSharedPtr<ICustomDetailsViewItem> FindItem(const FCustomDetailsViewItemId& InItemId) const = 0;

	/**
	 * Make a Sub Tree View for the Given Root Item Id
	 * @param InSourceItems pointer to the array of items that will be the top level items of the tree, these need to be valid for the duration of the Tree View lifetime
	 * @return the tree widget if the Root Item Id was found
	 */
	virtual TSharedRef<STreeView<TSharedPtr<ICustomDetailsViewItem>>> MakeSubTree(const TArray<TSharedPtr<ICustomDetailsViewItem>>* InSourceItems) const = 0;

	/** Request the Tree to be rebuilt with the given Build Type */
	virtual void RebuildTree(ECustomDetailsViewBuildType InBuildType) = 0;

	/** Adds a custom node into the hierarchy at the given position. */
	virtual void ExtendTree(FCustomDetailsViewItemId InHook, ECustomDetailsTreeInsertPosition InPosition, TSharedRef<ICustomDetailsViewItem> InItem) = 0;

	/** Retrieves all the extensions for the given item. */
	virtual const UE::CustomDetailsView::FTreeExtensionType& GetTreeExtensions(FCustomDetailsViewItemId InHook) const = 0;

	/** Creates a custom details view item based on a property handle. */
	virtual TSharedRef<ICustomDetailsViewItem> CreateDetailTreeItem(TSharedRef<ICustomDetailsViewItem> InParent, TSharedRef<IDetailTreeNode> InDetailTreeNode) = 0;

	/** Creates a custom details view item that has a customisable name and value widget. */
	virtual TSharedPtr<ICustomDetailsViewCustomItem> CreateCustomItem(TSharedRef<ICustomDetailsViewItem> InParent, FName InItemName, const FText& InLabel = FText::GetEmpty(), const FText& InToolTip = FText::GetEmpty()) = 0;

	/** Creates a custom details view item that acts as a category. */
	virtual TSharedPtr<ICustomDetailsViewCustomCategoryItem> CreateCustomCategoryItem(TSharedRef<ICustomDetailsViewItem> InParent, FName InItemName, const FText& InLabel = FText::GetEmpty(), const FText& InToolTip = FText::GetEmpty()) = 0;

	/** Find a previously added custom item. */
	virtual TSharedPtr<ICustomDetailsViewItem> FindCustomItem(const FName& InItemName) const = 0;

	/** Returns the current expansion state of an item. Returns true if the item state was found. */
	virtual bool GetItemExpansionState(const FCustomDetailsViewItemId& InItemId, ECustomDetailsViewExpansion& OutExpansion) const = 0;

	/** Sets the current expansion state of an item. */
	virtual void SetItemExpansionState(const FCustomDetailsViewItemId& InItemId, ECustomDetailsViewExpansion InExpansion) = 0;
};
