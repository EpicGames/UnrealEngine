// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailTreeNode.h"

class FCustomDetailsViewItemId
{
	template<typename T>
	static constexpr bool TFindPropertyCondition_V = TModels_V<CStaticClassProvider, T> || TModels_V<CStaticStructProvider, T>;

public:
	// This protects it from being taken by other new items added to EDetailNodeType.
	static constexpr uint32 CustomItemType = static_cast<uint32>(-1);
	static constexpr uint32 NullItemType = static_cast<uint32>(-2);

	FCustomDetailsViewItemId() = default;

	CUSTOMDETAILSVIEW_API static FCustomDetailsViewItemId MakeCategoryId(FName InCategoryName, 
		const FCustomDetailsViewItemId* InParentId = nullptr);
	CUSTOMDETAILSVIEW_API static FCustomDetailsViewItemId MakePropertyId(FProperty* Property);
	CUSTOMDETAILSVIEW_API static FCustomDetailsViewItemId MakePropertyId(const TSharedPtr<IPropertyHandle>& InPropertyHandle);
	CUSTOMDETAILSVIEW_API static FCustomDetailsViewItemId MakeCustomId(FName InItemName, 
		const FCustomDetailsViewItemId* InParentId = nullptr);

	template <
		typename InOwningStructType
		UE_REQUIRES(TFindPropertyCondition_V<InOwningStructType>)
	>
	static FCustomDetailsViewItemId MakePropertyId(FName InPropertyName)
	{
		if constexpr (TModels<CStaticClassProvider, InOwningStructType>::Value)
		{
			return FCustomDetailsViewItemId::MakePropertyId(FindFProperty<FProperty>(InOwningStructType::StaticClass(), InPropertyName));
		}
		else if constexpr (TModels<CStaticStructProvider, InOwningStructType>::Value)
		{
			return FCustomDetailsViewItemId::MakePropertyId(FindFProperty<FProperty>(InOwningStructType::StaticStruct(), InPropertyName));
		}
		else
		{
			return FCustomDetailsViewItemId();
		}
	}

	static FCustomDetailsViewItemId MakeFromDetailTreeNode(const TSharedRef<IDetailTreeNode>& InDetailTreeNode, 
		const FCustomDetailsViewItemId* InParentId = nullptr);

	friend uint32 GetTypeHash(const FCustomDetailsViewItemId& InItemId)
	{
		return InItemId.CachedHash;
	}

	bool operator==(const FCustomDetailsViewItemId& InOtherItemId) const
	{
		return CachedHash == InOtherItemId.CachedHash
			&& ItemType == InOtherItemId.ItemType
			&& ItemName == InOtherItemId.ItemName;
	}

	bool operator!=(const FCustomDetailsViewItemId& InOtherItemId) const
	{
		return !(*this == InOtherItemId);
	}

	const FString& GetItemName() const { return ItemName; }

	uint32 GetItemType() const { return ItemType; }

	bool IsType(EDetailNodeType InType) const;

private:
	FCustomDetailsViewItemId(uint32 InItemType, const FString& InItemName, const FCustomDetailsViewItemId* InParentId = nullptr);

	void CalculateTypeHash()
	{
		CachedHash = GetTypeHash(ItemName);
		CachedHash = HashCombine(CachedHash, ItemType);
	}

	FString ItemName;
	uint32  ItemType   = NullItemType;
	uint32  CachedHash = 0;
};
