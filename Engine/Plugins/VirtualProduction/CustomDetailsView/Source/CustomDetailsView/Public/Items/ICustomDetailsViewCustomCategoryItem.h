// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class ICustomDetailsViewItem;
class FText;
class SWidget;
enum class EDetailNodeType;

class ICustomDetailsViewCustomCategoryItem
{
public:
	virtual ~ICustomDetailsViewCustomCategoryItem() = default;

	/** Sets the text that will appear in the name column. */
	virtual void SetLabel(const FText& InLabel) = 0;

	/** Sets the tooltip that will appear in the name column. */
	virtual void SetToolTip(const FText& InToolTip) = 0;

	virtual TSharedRef<ICustomDetailsViewItem> AsItem() = 0;
};
