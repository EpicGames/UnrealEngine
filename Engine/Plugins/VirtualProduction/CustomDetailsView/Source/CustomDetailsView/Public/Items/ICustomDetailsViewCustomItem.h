// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class ICustomDetailsViewItem;
class FText;
class SWidget;
enum class EDetailNodeType;
template<typename OptionalType> struct TOptional;

class ICustomDetailsViewCustomItem
{
public:
	virtual ~ICustomDetailsViewCustomItem() = default;

	virtual void SetNodeType(TOptional<EDetailNodeType> InNodeType) = 0;

	/** Sets the text that will appear in the name column. Removes whole row widget override. */
	virtual void SetLabel(const FText& InLabel) = 0;

	/** Sets the tooltip that will appear in the name column. Removes whole row widget override. */
	virtual void SetToolTip(const FText& InToolTip) = 0;

	/** Sets the widget that will appear in the value column. Removes whole row widget override. */
	virtual void SetValueWidget(const TSharedRef<SWidget>& InValueWidget)  = 0;

	/** Sets the expansion widget override which where return to default, etc. are. Removes whole row widget override. */
	virtual void SetExpansionWidget(const TSharedRef<SWidget>& InExpansionWidget) = 0;

	/** Sets the whole row widget override and removes the other overrides. */
	virtual void SetWholeRowWidget(const TSharedRef<SWidget>& InWholeRowWidget) = 0;

	virtual TSharedRef<ICustomDetailsViewItem> AsItem() = 0;
};
