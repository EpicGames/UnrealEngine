// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewFwd.h"
#include "Templates/SharedPointer.h"

class FResetToDefaultOverride;
class ICustomDetailsView;
class IDetailTreeNode;
class SWidget;
enum class EDetailNodeType;

DECLARE_DELEGATE_RetVal_ThreeParams(
	TSharedPtr<ICustomDetailsViewItem>,
	FOnCustomDetailsViewGenerateChildItem,
	const TSharedRef<ICustomDetailsView>& InCustomDetailsView,
	const TSharedPtr<ICustomDetailsViewItem>& InParentItem,
	const TSharedRef<IDetailTreeNode>& InChildNode
)

DECLARE_DELEGATE_FourParams(
	FOnCustomDetailsViewCustomizeItemMenuContext,
	const TSharedRef<ICustomDetailsView>& InCustomDetailsView,
	const TSharedPtr<ICustomDetailsViewItem>& InItem,
	UObject* InMenuContext,
	TArray<TSharedPtr<IPropertyHandle>>& InPropertyHandles
)

class ICustomDetailsViewItem : public TSharedFromThis<ICustomDetailsViewItem>
{
public:
	virtual ~ICustomDetailsViewItem() = default;

	/** Called to initialise the widget. */
	virtual void InitWidget() = 0;

	virtual TSharedPtr<ICustomDetailsView> GetCustomDetailsView() const = 0;

	/** Details view containing this item */
	virtual TSharedPtr<IDetailsView> GetDetailsView() const = 0;

	/** Called to update the Item's Id */
	virtual void RefreshItemId() = 0;

	/** Retrieves the Item Id last updated from RefreshItemId, to avoid having to recalculate it every time */
	virtual const FCustomDetailsViewItemId& GetItemId() const  = 0;

	/** Regenerates the Children */
	virtual void RefreshChildren(TSharedPtr<ICustomDetailsViewItem> InParentOverride = nullptr) = 0;

	virtual TSharedPtr<ICustomDetailsViewItem> GetRoot() const = 0;

	virtual TSharedPtr<ICustomDetailsViewItem> GetParent() const = 0;

	virtual void SetParent(TSharedPtr<ICustomDetailsViewItem> InParent) = 0;

	virtual const TArray<TSharedPtr<ICustomDetailsViewItem>>& GetChildren() const = 0;

	virtual TOptional<EDetailNodeType> GetNodeType() const = 0;

	/** Adds this node as a child. */
	virtual void AddAsChild(const TSharedRef<ICustomDetailsViewItem>& InParentItem, TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) = 0;

	/**
	 * Instantiates a Widget for the Given Item
	 * @param InPrependWidget: Optional to prepend a Widget to the Name or Whole Row Widget
	 * @param InOwningWidget: Optional Widget to check for attributes like IsHovered()
	 */
	virtual TSharedRef<SWidget> MakeWidget(const TSharedPtr<SWidget>& InPrependWidget = nullptr
		, const TSharedPtr<SWidget>& InOwningWidget = nullptr) = 0;

	/**
	 * Get the one of the widgets that was generated in the MakeWidget
	 * Listen to the OnItemWidgetGenerated Delegate to have this Widget up to date with latest tree view
	 * @param InWidgetType: The type of widget to retrieve
	 */
	virtual TSharedPtr<SWidget> GetWidget(ECustomDetailsViewWidgetType InWidgetType) const = 0;

	/**
	 * Gets the widget set to override the widget automatically generated in the given slot.
	 */
	virtual TSharedPtr<SWidget> GetOverrideWidget(ECustomDetailsViewWidgetType InWidgetType) const = 0;

	/**
	 * Adds a widget to override an automatically generated widget for the given slot.
	 * @param InWidget The widget to override with, or a nullptr or SNullWidget::NullWidget to remove it.
	 */
	virtual void SetOverrideWidget(ECustomDetailsViewWidgetType InWidgetType, TSharedPtr<SWidget> InWidget) = 0;

	/** Override the keyframeability of this item. */
	virtual void SetKeyframeEnabled(bool bInKeyframeEnabled) = 0;

	/** Override the reset to the default information for this item. */
	virtual void SetResetToDefaultOverride(const FResetToDefaultOverride& InOverride) = 0;

	/** Checks to see if widget is visible */
	virtual bool IsWidgetVisible() const = 0;

	/** Overrides the created value widget's maximum width. */
	virtual void SetValueWidgetWidthOverride(TOptional<float> InWidth) = 0;

	/** Overrides the created widget's enabled status. */
	virtual void SetEnabledOverride(TAttribute<bool> InOverride) = 0;

	/** Returns the widget row that stores the default of this widget. */
	virtual const FDetailWidgetRow& GetDetailWidgetRow() const = 0;

	/** Creates the reset to default button based on this item's settings, if it can. */
	virtual bool CreateResetToDefaultButton(FPropertyRowExtensionButton& OutButton) = 0;

	/** Creates the other global extension buttons. */
	virtual void CreateGlobalExtensionButtons(TArray<FPropertyRowExtensionButton>& OutExtensionButtons) = 0;

	/** Takes a list of buttons and creates an extension button widget. */
	virtual TSharedRef<SWidget> CreateExtensionButtonWidget(const TArray<FPropertyRowExtensionButton>& InExtensionButtons) = 0;

	/**
	 * When the property based on a property row generator is expanded and child properties are generated, use
	 * this delegate to create the row, if set and it returns a non-null row.
	 */
	virtual void SetCreateChildItemDelegate(FOnCustomDetailsViewGenerateChildItem InDelegate) = 0;

	/**
	 * Allows to customize the context menu of this item
	 */
	virtual void SetCustomizeItemMenuContext(FOnCustomDetailsViewCustomizeItemMenuContext InDelegate) = 0;

protected:
	/** Adds the children of this node. */
	virtual void GatherChildren(const TSharedRef<ICustomDetailsViewItem>& InParentItem,
		const UE::CustomDetailsView::FTreeExtensionType& InTreeExtensions, ECustomDetailsTreeInsertPosition InPosition,
		TArray<TSharedPtr<ICustomDetailsViewItem>>& OutChildren) = 0;
};
